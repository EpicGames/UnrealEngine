{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Media Profile",
	"Description": "This plugin contains the Media Profile asset and related entities, which help manage media sources and outputs",
	"Category": "Media Players",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "MediaProfile",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Mac"
			]
		},
		{
			"Name": "MediaProfileEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Mac"
			]
		}
	],
	"Plugins": [
		{
			"Name": "MediaIOFramework",
			"Enabled": true
		},
		{
			"Name": "MediaPlayerEditor",
			"Enabled": true
		}
	]
}