// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MediaProfile : ModuleRules
	{
		public MediaProfile(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"Engine"
				});
				
			
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CoreUObject",
					"MediaAssets",
					"MediaIOCore"
				});
			
			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"EditorFramework",
						"LevelEditor",
						"MediaPlayerEditor",
						"Slate",
						"SlateCore",
						"Settings",
						"UnrealEd"
					});
			}
		}
	}
}
