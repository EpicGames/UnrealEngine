// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaAssets/ProxyMediaOutput.h"
#include "MediaProfileModule.h"


/* UProxyMediaOutput structors
 *****************************************************************************/

 
UProxyMediaOutput::UProxyMediaOutput()
	: bLeafMediaOutput(false)
	, bValidateGuard(false)
	, bRequestedSizeGuard(false)
	, bRequestedPixelFormatGuard(false)
	, bCreateMediaCaptureImplGuard(false)
{}


/* UMediaOutput interface
 *****************************************************************************/

 
bool UProxyMediaOutput::Validate(FString& OutFailureReason) const
{
	// Guard against reentrant calls.
	if (bValidateGuard)
	{
		UE_LOGF(LogMediaProfile, Warning, "UProxyMediaOutput::Validate - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		OutFailureReason = TEXT("Reentrant calls");
		return false;
	}
	TGuardValue<bool> ValidatingGuard(bValidateGuard, true);

	UMediaOutput* CurrentProxy = GetMediaOutput();
	if (CurrentProxy != nullptr)
	{
		return CurrentProxy->Validate(OutFailureReason);
	}
	
	OutFailureReason = FString::Printf(TEXT("There is no proxy for MediaOutput '%s'."), *GetName());
	return false;
}


FIntPoint UProxyMediaOutput::GetRequestedSize() const
{
	// Guard against reentrant calls.
	if (bRequestedSizeGuard)
	{
		UE_LOGF(LogMediaProfile, Warning, "UProxyMediaOutput::GetRequestedSize - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return FIntPoint::ZeroValue;
	}
	TGuardValue<bool> GettingUrlGuard(bRequestedSizeGuard, true);

	UMediaOutput* CurrentProxy = GetMediaOutput();
	return (CurrentProxy != nullptr) ? CurrentProxy->GetRequestedSize() : FIntPoint::ZeroValue;
}


EPixelFormat UProxyMediaOutput::GetRequestedPixelFormat() const
{
	// Guard against reentrant calls.
	if (bRequestedPixelFormatGuard)
	{
		UE_LOGF(LogMediaProfile, Warning, "UProxyMediaOutput::GetRequestedPixelFormat - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return EPixelFormat::PF_Unknown;
	}
	TGuardValue<bool> GettingUrlGuard(bRequestedPixelFormatGuard, true);

	UMediaOutput* CurrentProxy = GetMediaOutput();
	return (CurrentProxy != nullptr) ? CurrentProxy->GetRequestedPixelFormat() : EPixelFormat::PF_Unknown;
}


EMediaCaptureConversionOperation UProxyMediaOutput::GetConversionOperation(EMediaCaptureSourceType InSourceType) const
{
	// Guard against reentrant calls.
	if (bRequestedPixelFormatGuard)
	{
		UE_LOGF(LogMediaProfile, Warning, "UProxyMediaOutput::GetConversionOperation - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return EMediaCaptureConversionOperation::NONE;
	}
	TGuardValue<bool> GettingUrlGuard(bRequestedPixelFormatGuard, true);

	UMediaOutput* CurrentProxy = GetMediaOutput();
	return (CurrentProxy != nullptr) ? CurrentProxy->GetConversionOperation(InSourceType) : EMediaCaptureConversionOperation::NONE;
}

#if WITH_EDITOR
FString UProxyMediaOutput::GetDescriptionString() const
{
	// Guard against reentrant calls.
	if (bConfigurationTextGuard)
	{
		UE_LOGF(LogMediaProfile, Warning, "UProxyMediaOutput::GetConfigurationDisplayText - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return FString();
	}
	TGuardValue<bool> Guard(bConfigurationTextGuard, true);

	UMediaOutput* CurrentProxy = GetMediaOutput();
	return (CurrentProxy != nullptr) ? CurrentProxy->GetDescriptionString() : FString();
}

void UProxyMediaOutput::GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const
{
	// Guard against reentrant calls.
	if (bGetInfoElementsGuard)
	{
		UE_LOGF(LogMediaProfile, Warning, "UProxyMediaOutput::GetDetailsPanelInfoElements - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return;
	}
	TGuardValue<bool> Guard(bGetInfoElementsGuard, true);

	UMediaOutput* CurrentProxy = GetMediaOutput();
	if (CurrentProxy != nullptr)
	{
		CurrentProxy->GetDetailsPanelInfoElements(OutInfoElements);
	}
}
#endif //WITH_EDITOR

UMediaCapture* UProxyMediaOutput::CreateMediaCaptureImpl()
{
	// Guard against reentrant calls.
	if (bCreateMediaCaptureImplGuard)
	{
		UE_LOGF(LogMediaProfile, Warning, "UProxyMediaOutput::CreateMediaCaptureImpl - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return nullptr;
	}
	TGuardValue<bool> GettingUrlGuard(bCreateMediaCaptureImplGuard, true);

	UMediaOutput* CurrentProxy = GetMediaOutput();
	return (CurrentProxy != nullptr) ? CurrentProxy->CreateMediaCapture() : nullptr;
}


/* UProxyMediaOutput implementation
 *****************************************************************************/

UMediaOutput* UProxyMediaOutput::GetMediaOutput() const
{
	return DynamicProxy ? DynamicProxy : Proxy;
}


UMediaOutput* UProxyMediaOutput::GetLeafMediaOutput() const
{
	// Guard against reentrant calls.
	if (bLeafMediaOutput)
	{
		UE_LOGF(LogMediaProfile, Warning, "UMediaSourceProxy::GetLeafMediaOutput - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return nullptr;
	}
	TGuardValue<bool> ValidatingGuard(bLeafMediaOutput, true);

	UMediaOutput* MediaOutput = GetMediaOutput();
	if (UProxyMediaOutput* ProxyMediaOutput = Cast<UProxyMediaOutput>(MediaOutput))
	{
		MediaOutput = ProxyMediaOutput->GetLeafMediaOutput();
	}
	return MediaOutput;
}


bool UProxyMediaOutput::IsProxyValid() const
{
	return GetLeafMediaOutput() != nullptr;
}


void UProxyMediaOutput::SetDynamicMediaOutput(UMediaOutput* InProxy)
{
	DynamicProxy = InProxy;
}


#if WITH_EDITOR
void UProxyMediaOutput::SetMediaOutput(UMediaOutput* InProxy)
{
	Proxy = InProxy;
}
#endif