// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaAssets/ProxyMediaSource.h"
#include "MediaProfileModule.h"

 
/* UMediaSourceProxy structors
 *****************************************************************************/
 
 
UProxyMediaSource::UProxyMediaSource()
	: bUrlGuard(false)
	, bValidateGuard(false)
	, bLeafMediaSource(false)
	, bMediaOptionGuard(false)
{}


/* UMediaSource interface
 *****************************************************************************/
 
 
FString UProxyMediaSource::GetUrl() const
{
	// Guard against reentrant calls.
	if (bUrlGuard)
	{
		UE_LOGF(LogMediaProfile, Warning, "UMediaSourceProxy::GetUrl - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return FString();
	}
	TGuardValue<bool> GettingUrlGuard(bUrlGuard, true);

	UMediaSource* CurrentProxy = GetMediaSource();
	return (CurrentProxy != nullptr) ? CurrentProxy->GetUrl() : FString();
}


bool UProxyMediaSource::Validate() const
{
	// Guard against reentrant calls.
	if (bValidateGuard)
	{
		UE_LOGF(LogMediaProfile, Warning, "UMediaSourceProxy::Validate - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return false;
	}
	TGuardValue<bool> ValidatingGuard(bValidateGuard, true);

	UMediaSource* CurrentProxy = GetMediaSource();
	return (CurrentProxy != nullptr) ? CurrentProxy->Validate() : false;
}

#if WITH_EDITOR
FString UProxyMediaSource::GetDescriptionString() const
{
	// Guard against reentrant calls.
	if (bConfigurationTextGuard)
	{
		UE_LOGF(LogMediaProfile, Warning, "UMediaSourceProxy::GetConfigurationDisplayText - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return FString();
	}
	TGuardValue<bool> Guard(bConfigurationTextGuard, true);

	UMediaSource* CurrentProxy = GetMediaSource();
	return (CurrentProxy != nullptr) ? CurrentProxy->GetDescriptionString() : FString();
}

void UProxyMediaSource::GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const
{
	// Guard against reentrant calls.
	if (bGetInfoElementsGuard)
	{
		UE_LOGF(LogMediaProfile, Warning, "UMediaSourceProxy::GetDetailsPanelInfoElements - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return;
	}
	TGuardValue<bool> Guard(bGetInfoElementsGuard, true);

	UMediaSource* CurrentProxy = GetMediaSource();
	if (CurrentProxy != nullptr)
	{
		CurrentProxy->GetDetailsPanelInfoElements(OutInfoElements);
	}
}
#endif //WITH_EDITOR

/* UMediaSourceProxy implementation
 *****************************************************************************/

UMediaSource* UProxyMediaSource::GetMediaSource() const
{
	return DynamicProxy ? DynamicProxy : Proxy;
}


UMediaSource* UProxyMediaSource::GetLeafMediaSource() const
{
	// Guard against reentrant calls.
	if (bLeafMediaSource)
	{
		UE_LOGF(LogMediaProfile, Warning, "UMediaSourceProxy::GetLeafMediaSource - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return nullptr;
	}
	TGuardValue<bool> ValidatingGuard(bLeafMediaSource, true);

	UMediaSource* MediaSource = GetMediaSource();
	if (UProxyMediaSource* ProxyMediaSource = Cast<UProxyMediaSource>(MediaSource))
	{
		MediaSource = ProxyMediaSource->GetLeafMediaSource();
	}
	return MediaSource;
}


bool UProxyMediaSource::IsProxyValid() const
{
	return GetLeafMediaSource() != nullptr;
}


void UProxyMediaSource::SetDynamicMediaSource(UMediaSource* InProxy)
{
	DynamicProxy = InProxy;
}


#if WITH_EDITOR
void UProxyMediaSource::SetMediaSource(UMediaSource* InProxy)
{
	Proxy = InProxy;
}
#endif


/* IMediaOptions interface
 *****************************************************************************/

bool UProxyMediaSource::GetMediaOption(const FName& Key, bool DefaultValue) const
{
	// Guard against reentrant calls.
	if (bMediaOptionGuard)
	{
		UE_LOGF(LogMediaProfile, Warning, "UMediaSourceProxy::GetMediaOption - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return DefaultValue;
	}
	TGuardValue<bool> GettingOptionGuard(bMediaOptionGuard, true);

	UMediaSource* CurrentProxy = GetMediaSource();
	if (CurrentProxy != nullptr)
	{
		return CurrentProxy->GetMediaOption(Key, DefaultValue);
	}

	return Super::GetMediaOption(Key, DefaultValue);
}


double UProxyMediaSource::GetMediaOption(const FName& Key, double DefaultValue) const
{
	// Guard against reentrant calls.
	if (bMediaOptionGuard)
	{
		UE_LOGF(LogMediaProfile, Warning, "UMediaSourceProxy::GetMediaOption - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return DefaultValue;
	}
	TGuardValue<bool> GettingOptionGuard(bMediaOptionGuard, true);

	UMediaSource* CurrentProxy = GetMediaSource();
	if (CurrentProxy != nullptr)
	{
		return CurrentProxy->GetMediaOption(Key, DefaultValue);
	}

	return Super::GetMediaOption(Key, DefaultValue);
}


int64 UProxyMediaSource::GetMediaOption(const FName& Key, int64 DefaultValue) const
{
	// Guard against reentrant calls.
	if (bMediaOptionGuard)
	{
		UE_LOGF(LogMediaProfile, Warning, "UMediaSourceProxy::GetMediaOption - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return DefaultValue;
	}
	TGuardValue<bool> GettingOptionGuard(bMediaOptionGuard, true);

	UMediaSource* CurrentProxy = GetMediaSource();
	if (CurrentProxy != nullptr)
	{
		return CurrentProxy->GetMediaOption(Key, DefaultValue);
	}

	return Super::GetMediaOption(Key, DefaultValue);
}


FString UProxyMediaSource::GetMediaOption(const FName& Key, const FString& DefaultValue) const
{
	// Guard against reentrant calls.
	if (bMediaOptionGuard)
	{
		UE_LOGF(LogMediaProfile, Warning, "UMediaSourceProxy::GetMediaOption - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return DefaultValue;
	}
	TGuardValue<bool> GettingOptionGuard(bMediaOptionGuard, true);

	UMediaSource* CurrentProxy = GetMediaSource();
	if (CurrentProxy != nullptr)
	{
		return CurrentProxy->GetMediaOption(Key, DefaultValue);
	}

	return Super::GetMediaOption(Key, DefaultValue);
}


FText UProxyMediaSource::GetMediaOption(const FName& Key, const FText& DefaultValue) const
{
	// Guard against reentrant calls.
	if (bMediaOptionGuard)
	{
		UE_LOGF(LogMediaProfile, Warning, "UMediaSourceProxy::GetMediaOption - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return DefaultValue;
	}
	TGuardValue<bool> GettingOptionGuard(bMediaOptionGuard, true);

	UMediaSource* CurrentProxy = GetMediaSource();
	if (CurrentProxy != nullptr)
	{
		return CurrentProxy->GetMediaOption(Key, DefaultValue);
	}

	return Super::GetMediaOption(Key, DefaultValue);
}

TSharedPtr<IMediaOptions::FDataContainer, ESPMode::ThreadSafe> UProxyMediaSource::GetMediaOption(const FName& Key, const TSharedPtr<FDataContainer, ESPMode::ThreadSafe>& DefaultValue) const
{
	// Guard against reentrant calls.
	if (bMediaOptionGuard)
	{
		UE_LOGF(LogMediaProfile, Warning, "UMediaSourceProxy::GetMediaOption - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return DefaultValue;
	}
	TGuardValue<bool> GettingOptionGuard(bMediaOptionGuard, true);

	UMediaSource* CurrentProxy = GetMediaSource();
	if (CurrentProxy != nullptr)
	{
		return CurrentProxy->GetMediaOption(Key, DefaultValue);
	}

	return Super::GetMediaOption(Key, DefaultValue);
}


bool UProxyMediaSource::HasMediaOption(const FName& Key) const
{
	// Guard against reentrant calls.
	if (bMediaOptionGuard)
	{
		UE_LOGF(LogMediaProfile, Warning, "UMediaSourceProxy::HasMediaOption - Reentrant calls are not supported. Asset: %ls", *GetPathName());
		return false;
	}
	TGuardValue<bool> HasOptionGuard(bMediaOptionGuard, true);

	UMediaSource* CurrentProxy = GetMediaSource();
	if (CurrentProxy != nullptr)
	{
		return CurrentProxy->HasMediaOption(Key);
	}

	return Super::HasMediaOption(Key);
}
