// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaProfileModule.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/Engine.h"
#include "MediaAssets/ProxyMediaSource.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Profile/MediaProfile.h"
#include "Profile/MediaProfileSettings.h"

#if WITH_EDITOR
#include "ISettingsModule.h"
#include "LevelEditor.h"
#endif //WITH_EDITOR

#define LOCTEXT_NAMESPACE "FMediaProfileModule"

DEFINE_LOG_CATEGORY(LogMediaProfile);

const FName IMediaProfileModule::LegacyPackageName(TEXT("/Script/MediaFrameworkUtilities"));

static TAutoConsoleVariable<FString> CVarMediaUtilsStartupProfile(
	TEXT("MediaUtils.StartupProfile"),
	TEXT(""),
	TEXT("Startup Media Profile\n"),
	ECVF_ReadOnly
);

static FAutoConsoleCommand MediaProfileSetCmd(
	TEXT("MediaProfile.Set"),
	TEXT("Set the current Media Profile. Usage: MediaProfile.Set /Game/Path/To/MediaProfile.MediaProfile (or no argument to clear)"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (Args.Num() == 0)
		{
			// Clear the media profile when called with no arguments
			if (IMediaProfileModule* Module = FModuleManager::GetModulePtr<IMediaProfileModule>("MediaProfile"))
			{
				Module->GetProfileManager().SetCurrentMediaProfile(nullptr);
				UE_LOGF(LogMediaProfile, Display, "Cleared Media Profile");
			}
			return;
		}

		const FString& ProfilePath = Args[0];

		if (UObject* Object = StaticLoadObject(UMediaProfile::StaticClass(), nullptr, *ProfilePath))
		{
			if (UMediaProfile* MediaProfile = Cast<UMediaProfile>(Object))
			{
				if (IMediaProfileModule* Module = FModuleManager::GetModulePtr<IMediaProfileModule>("MediaProfile"))
				{
					Module->GetProfileManager().SetCurrentMediaProfile(MediaProfile);
					UE_LOGF(LogMediaProfile, Display, "Applied Media Profile: %ls", *ProfilePath);
				}
			}
			else
			{
				UE_LOGF(LogMediaProfile, Error, "Object at path '%ls' is not a Media Profile", *ProfilePath);
			}
		}
		else
		{
			UE_LOGF(LogMediaProfile, Error, "Failed to load Media Profile from path: %ls", *ProfilePath);
		}
	})
);

static FAutoConsoleCommand MediaProfileClearCmd(
	TEXT("MediaProfile.Clear"),
	TEXT("Clear the current Media Profile"),
	FConsoleCommandDelegate::CreateLambda([]()
	{
		if (IMediaProfileModule* Module = FModuleManager::GetModulePtr<IMediaProfileModule>("MediaProfile"))
		{
			Module->GetProfileManager().SetCurrentMediaProfile(nullptr);
			UE_LOGF(LogMediaProfile, Display, "Cleared Media Profile");
		}
	})
);

namespace UE::MediaProfile::Private
{
	/**
	 * Copy every key under OldSectionName into the section whose name matches the current class
	 * path in the in-memory merged config. UObject::LoadConfig only reads GetPathName()'s section
	 * and does not consult FCoreRedirects, so legacy project configs that still reference
	 * /Script/MediaFrameworkUtilities.* would otherwise be ignored.
	 */
	void FixupConfig(const UClass* InConfigClass, const FName& InOldPackageName)
	{
		if (!GConfig)
		{
			return;
		}

		const FString ConfigFilename = InConfigClass->GetConfigName();
		if (ConfigFilename.IsEmpty())
		{
			return;
		}

		const FString OldSectionName = InOldPackageName.ToString() + TEXT(".") + InConfigClass->GetName();
		const FConfigSection* LegacySection = GConfig->GetSection(*OldSectionName, false, ConfigFilename);
		if (!LegacySection)
		{
			return;
		}

		const FString NewSectionName = InConfigClass->GetPathName();
		for (FConfigSection::TConstIterator It(*LegacySection); It; ++It)
		{
			GConfig->AddUniqueToSection(*NewSectionName, It.Key(), It.Value().GetSavedValue(), ConfigFilename);
		}
	}
}

void FMediaProfileModule::StartupModule()
{	
	RegisterSettings();
	ApplyStartupMediaProfile();

#if WITH_EDITOR
	if (GIsEditor && FModuleManager::Get().IsModuleLoaded("LevelEditor"))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		bIsLevelEditorInitialized = LevelEditorModule.GetFirstLevelEditor().IsValid();
		OnLevelEditorCreatedHandle = LevelEditorModule.OnLevelEditorCreated().AddLambda(
			[this](TSharedPtr<ILevelEditor> InLevelEditor)
			{
				bIsLevelEditorInitialized = true;
			}
		);
	}
#endif
}

void FMediaProfileModule::ShutdownModule()
{
	UnregisterSettings();
	RemoveStartupMediaProfile();
	
#if WITH_EDITOR
	bIsLevelEditorInitialized = false;
	if (GIsEditor && OnLevelEditorCreatedHandle.IsValid())
	{
		if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
		{
			LevelEditorModule->OnLevelEditorCreated().Remove(OnLevelEditorCreatedHandle);
		}
	}
#endif
}

void FMediaProfileModule::RegisterSettings()
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		// register settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
		if (SettingsModule != nullptr)
		{
			SettingsModule->RegisterSettings("Project", "Plugins", "MediaProfile",
				LOCTEXT("MediaProfilesSettingsName", "Media Profile"),
				LOCTEXT("MediaProfilesDescription", "Configure the Media Profile."),
				GetMutableDefault<UMediaProfileSettings>()
			);

			SettingsModule->RegisterSettings("Editor", "General", "MediaProfile",
				LOCTEXT("MediaProfilesSettingsName", "Media Profile"),
				LOCTEXT("MediaProfilesDescription", "Configure the Media Profile."),
				GetMutableDefault<UMediaProfileEditorSettings>()
			);
		}
	}
#endif //WITH_EDITOR
}

void FMediaProfileModule::UnregisterSettings()
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		// unregister settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Media", "MediaProfile");
			SettingsModule->UnregisterSettings("Editor", "Media", "MediaProfile");
		}
	}
#endif //WITH_EDITOR
}

void FMediaProfileModule::ApplyStartupMediaProfile()
{
	auto ApplyMediaProfile = [this]()
	{
		UMediaProfile* MediaProfile = nullptr;

		// Try to load from CVar
		{
			const FString MediaProfileName = CVarMediaUtilsStartupProfile.GetValueOnGameThread();

			if (MediaProfileName.Len())
			{
				if (UObject* Object = StaticLoadObject(UMediaProfile::StaticClass(), nullptr, *MediaProfileName))
				{
					MediaProfile = CastChecked<UMediaProfile>(Object);
				}

				if (MediaProfile)
				{
					UE_LOGF(LogMediaProfile, Display,
						"Loading Media Profile specified in CVar MediaUtils.StartupProfile: '%ls'", *MediaProfileName);
				}
			}
		}

#if WITH_EDITOR
		// Try to load from User Settings
		if (MediaProfile == nullptr)
		{
			MediaProfile = GetDefault<UMediaProfileEditorSettings>()->GetUserMediaProfile();
		}
#endif

		// Try to load from Game Settings
		if (MediaProfile == nullptr)
		{
			MediaProfile = GetDefault<UMediaProfileSettings>()->GetStartupMediaProfile();
		}

#if WITH_EDITOR
		if (MediaProfile)
		{
			GetMutableDefault<UMediaProfileSettings>()->FillDefaultMediaSourceProxies(MediaProfile->NumMediaSources(), true);
			GetMutableDefault<UMediaProfileSettings>()->FillDefaultMediaOutputProxies(MediaProfile->NumMediaOutputs(), false);
		}
#endif
		// Only override the current profile if we found a startup profile to load.
		// If no startup profile is configured, leave whatever is already set (e.g. a
		// transient profile created by another system) alone.
		if (MediaProfile)
		{
			MediaProfileManager.SetCurrentMediaProfile(MediaProfile);
		}
	};

	if (FApp::CanEverRender() || GetDefault<UMediaProfileSettings>()->bApplyInCommandlet)
	{
		check(!(GEngine && GEngine->IsInitialized()));
		InitHandle = FCoreDelegates::OnFEngineLoopInitComplete.AddLambda(ApplyMediaProfile);
	}
}

void FMediaProfileModule::RemoveStartupMediaProfile()
{
	if (InitHandle.IsValid())
	{
		FCoreDelegates::OnFEngineLoopInitComplete.Remove(InitHandle);
	}

	if (!IsEngineExitRequested())
	{
		MediaProfileManager.SetCurrentMediaProfile(nullptr);
	}
}

void FMediaProfileModule::FixupRedirectedConfigs()
{
	static const TArray<UClass*> ConfigClassesToRedirect = 
	{
		UMediaProfileSettings::StaticClass(),
		UMediaProfileEditorSettings::StaticClass()
	};

	for (const UClass* Class : ConfigClassesToRedirect)
	{
		UE::MediaProfile::Private::FixupConfig(Class, IMediaProfileModule::LegacyPackageName);
	}
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FMediaProfileModule, MediaProfile)