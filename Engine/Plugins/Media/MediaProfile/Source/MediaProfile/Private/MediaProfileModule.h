// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaProfileModule.h"
#include "Modules/ModuleManager.h"
#include "Profile/MediaProfileManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMediaProfile, Log, All);

class FMediaProfileModule : public IMediaProfileModule
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
	virtual IMediaProfileManager& GetProfileManager() override { return MediaProfileManager;}
	virtual bool CanCaptureWithLevelEditorViewportClients() override { return bIsLevelEditorInitialized;}
	
private:
	void RegisterSettings();
	void UnregisterSettings();
	
	void ApplyStartupMediaProfile();
	void RemoveStartupMediaProfile();
	
	/** Fixes up all saved config properties for UMediaProfileSettings and UMediaProfileUserSettings which may still be saved under the old MFU package name */
	void FixupRedirectedConfigs();
	
private:
	/** Keeps track of whether the level editor module has been initialized and can be used for media capture */
	bool bIsLevelEditorInitialized = false;
	
	FMediaProfileManager MediaProfileManager;
	FDelegateHandle InitHandle;
	FDelegateHandle OnLevelEditorCreatedHandle;
};
