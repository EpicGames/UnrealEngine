// Copyright Epic Games, Inc. All Rights Reserved.


#include "Profile/MediaProfile.h"

#include "IAnalyticsProviderET.h"
#include "MediaProfileModule.h"

#include "Engine/Engine.h"
#include "Engine/EngineCustomTimeStep.h"
#include "Engine/TimecodeProvider.h"
#include "MediaAssets/ProxyMediaOutput.h"
#include "MediaAssets/ProxyMediaSource.h"
#include "Profile/IMediaProfileManager.h"
#include "Profile/MediaProfilePlaybackManager.h"

#if WITH_EDITOR
#include "EngineAnalytics.h"
#endif

#if WITH_EDITOR
namespace MediaProfileAnalytics
{
	template <typename ObjectType>
	auto JoinObjectNames = [](const TArray<ObjectType*>& InObjects) -> FString
	{
		TArray<FString> SourceNames;
		Algo::TransformIf(
			InObjects,
			SourceNames,
			[](ObjectType* Object)
			{
				return !!Object;
			},
			[](ObjectType* Object)
			{
				return Object->GetName();
			});

		TStringBuilder<64> StringBuilder;
		StringBuilder.Join(SourceNames, TEXT(","));
		return FString(StringBuilder);
	};
}

namespace UE::MediaProfile::Private
{
	FString GetUniqueName(const FString& InDesiredName, const TArray<FString>& ExistingNames)
	{
		FString CurrentName = InDesiredName;
		int32 Index = 1;
		while (ExistingNames.Contains(CurrentName))
		{
			CurrentName = InDesiredName + TEXT("_") + FString::FromInt(Index);
		}
		
		return CurrentName;
	}
}
#endif

UMediaProfile::UMediaProfile(const FObjectInitializer& ObjectInitializer)
{
	PlaybackManager = CreateDefaultSubobject<UMediaProfilePlaybackManager>(TEXT("PlaybackManager"), true);
}

UMediaSource* UMediaProfile::GetMediaSource(int32 Index) const
{
	if (MediaSources.IsValidIndex(Index))
	{
		return MediaSources[Index];
	}
	return nullptr;
}

void UMediaProfile::SetMediaSource(int32 Index, UMediaSource* InMediaSource)
{
	if (MediaSources.IsValidIndex(Index))
	{
		FString CurrentMediaSourceName = MediaSources[Index] ? MediaSources[Index]->GetName() : TEXT("");
		MediaSources[Index] = InMediaSource;
		
#if WITH_EDITOR
		FPropertyChangedEvent PropertyChangedEvent(StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMediaProfile, MediaSources)), EPropertyChangeType::ValueSet);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(this, PropertyChangedEvent);
		
		// If the original label for the media source was the source's object name (indicating the user has not changed it)
		// update the label to the new source's object name
		if (!MediaSourceLabels.IsValidIndex(Index) || GetLabelForMediaSource(Index) == CurrentMediaSourceName)
		{
			SetLabelForMediaSource(Index, InMediaSource->GetName());
		}
#endif
	}
}

int32 UMediaProfile::NumMediaSources() const
{
	return MediaSources.Num();
}

int32 UMediaProfile::FindMediaSourceIndex(UMediaSource* MediaSource) const
{
	return MediaSources.Find(MediaSource);
}

#if WITH_EDITOR
int32 UMediaProfile::FindMediaSourceIndexByLabel(const FString& InLabel) const
{
	int32 Index = MediaSourceLabels.Find(InLabel);
	
	if (MediaSources.IsValidIndex(Index))
	{
		return Index;
	}
	
	return INDEX_NONE;
}
#endif

void UMediaProfile::AddMediaSource(UMediaSource* MediaSource)
{
	MediaSources.Add(MediaSource);

#if WITH_EDITOR
	if (MediaSourceLabels.Num() < MediaSources.Num())
	{
		for (int32 Index = MediaSourceLabels.Num(); Index < MediaSources.Num(); ++Index)
		{
			FString Name = MediaSources[Index] ? MediaSources[Index]->GetName() : TEXT("");
			MediaSourceLabels.Add(Name);
		}
	}
	else
	{
		FString Name = MediaSource ? MediaSource->GetName() : TEXT("");
		const int32 Index = MediaSources.Num() - 1;
		if (MediaSourceLabels.IsValidIndex(Index))
		{
			MediaSourceLabels[Index] = Name;
		}
		else
		{
			MediaSourceLabels.Add(Name);
		}
	}
#endif
}

bool UMediaProfile::RemoveMediaSource(int32 Index)
{
	if (!MediaSources.IsValidIndex(Index))
	{
		return false;
	}
	
	PlaybackManager->ChangeMediaSourceIndex(Index, INDEX_NONE);
	MediaSources.RemoveAt(Index, EAllowShrinking::No);

#if WITH_EDITOR
	if (MediaSourceLabels.IsValidIndex(Index))
	{
		MediaSourceLabels.RemoveAt(Index, EAllowShrinking::No);
	}
#endif
	
	return true;
}

bool UMediaProfile::MoveMediaSource(int32 CurrentIndex, int32 DestIndex)
{
	if (!MediaSources.IsValidIndex(CurrentIndex) || !MediaSources.IsValidIndex(DestIndex))
	{
		return false;
	}

	PlaybackManager->ChangeMediaSourceIndex(CurrentIndex, DestIndex);
	TObjectPtr<UMediaSource> MediaSource = MediaSources[CurrentIndex];
	MediaSources.RemoveAt(CurrentIndex, EAllowShrinking::No);
	MediaSources.Insert(MediaSource, DestIndex);

#if WITH_EDITOR
	if (MediaSourceLabels.IsValidIndex(CurrentIndex))
	{
		FString Label = MediaSourceLabels[CurrentIndex];
		MediaSourceLabels.RemoveAt(CurrentIndex, EAllowShrinking::No);
		MediaSourceLabels.Insert(Label, DestIndex);
	}
#endif
	
	return true;
}

#if WITH_EDITOR
FString UMediaProfile::GetLabelForMediaSource(int32 Index)
{
	if (!MediaSourceLabels.IsValidIndex(Index))
	{
		if (MediaSources.IsValidIndex(Index))
		{
			if (MediaSources[Index])
			{
				return MediaSources[Index]->GetName();
			}
			else
			{
				return TEXT("");
			}
		}
		
		return TEXT("");
	}
	
	return MediaSourceLabels[Index];
}

void UMediaProfile::SetLabelForMediaSource(int32 Index, const FString& NewLabel)
{
	if (!MediaSourceLabels.IsValidIndex(Index))
	{
		if (!MediaSources.IsValidIndex(Index))
		{
			return;
		}
		
		// In some cases, we may not have labels for all media sources yet (such as an older media profile being edited)
		// In that case, add all the missing labels now
		for (int32 MissingIdx = MediaSourceLabels.Num(); MissingIdx < MediaSources.Num(); ++MissingIdx)
		{
			if (MediaSources[MissingIdx])
			{
				MediaSourceLabels.Add(MediaSources[MissingIdx]->GetName());
			}
			else
			{
				MediaSourceLabels.Add(TEXT(""));
			}
		}
	}
	
	MediaSourceLabels[Index] = UE::MediaProfile::Private::GetUniqueName(NewLabel, MediaSourceLabels);
	
	FPropertyChangedEvent PropertyChangedEvent(StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMediaProfile, MediaSourceLabels)), EPropertyChangeType::ValueSet);
	FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(this, PropertyChangedEvent);
}
#endif //WITH_EDITOR

UMediaOutput* UMediaProfile::GetMediaOutput(int32 Index) const
{
	if (MediaOutputs.IsValidIndex(Index))
	{
		return MediaOutputs[Index];
	}
	return nullptr;
}

void UMediaProfile::SetMediaOutput(int32 Index, UMediaOutput* InMediaOutput)
{
	if (MediaOutputs.IsValidIndex(Index))
	{
		FString CurrentMediaOutputName = MediaOutputs[Index] ? MediaOutputs[Index]->GetName() : TEXT("");
		MediaOutputs[Index] = InMediaOutput;
		
#if WITH_EDITOR
		FPropertyChangedEvent PropertyChangedEvent(StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMediaProfile, MediaOutputs)), EPropertyChangeType::ValueSet);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(this, PropertyChangedEvent);
		
		// If the original label for the media output was the output's object name (indicating the user has not changed it)
		// update the label to the new output's object name
		if (!MediaOutputLabels.IsValidIndex(Index) ||GetLabelForMediaOutput(Index) == CurrentMediaOutputName)
		{
			SetLabelForMediaOutput(Index, InMediaOutput->GetName());
		}
#endif
	}
}

int32 UMediaProfile::NumMediaOutputs() const
{
	return MediaOutputs.Num();
}

int32 UMediaProfile::FindMediaOutputIndex(UMediaOutput* MediaOutput) const
{
	return MediaOutputs.Find(MediaOutput);
}

#if WITH_EDITOR
int32 UMediaProfile::FindMediaOutputIndexByLabel(const FString& InLabel) const
{
	int32 Index = MediaOutputLabels.Find(InLabel);
	
	if (MediaOutputs.IsValidIndex(Index))
	{
		return Index;
	}
	
	return INDEX_NONE;
}
#endif

void UMediaProfile::AddMediaOutput(UMediaOutput* MediaOutput)
{
	MediaOutputs.Add(MediaOutput);

#if WITH_EDITOR
	if (MediaOutputLabels.Num() < MediaOutputs.Num())
	{
		for (int32 Index = MediaOutputLabels.Num(); Index < MediaOutputs.Num(); ++Index)
		{
			FString Name = MediaOutputs[Index] ? MediaOutputs[Index]->GetName() : TEXT("");
			MediaOutputLabels.Add(Name);
		}
	}
	else
	{
		FString Name = MediaOutput ? MediaOutput->GetName() : TEXT("");
		const int32 Index = MediaOutputs.Num() - 1;
		if (MediaOutputLabels.IsValidIndex(Index))
		{
			MediaOutputLabels[Index] = Name;
		}
		else
		{
			MediaOutputLabels.Add(Name);
		}
	}
#endif
}

bool UMediaProfile::RemoveMediaOutput(int32 Index)
{
	if (!MediaOutputs.IsValidIndex(Index))
	{
		return false;
	}

	PlaybackManager->ChangeMediaOutputIndex(Index, INDEX_NONE);
	MediaOutputs.RemoveAt(Index, EAllowShrinking::No);

#if WITH_EDITOR
	if (MediaOutputLabels.IsValidIndex(Index))
	{
		MediaOutputLabels.RemoveAt(Index, EAllowShrinking::No);
	}
#endif
	
	return true;
}

bool UMediaProfile::MoveMediaOutput(int32 CurrentIndex, int32 DestIndex)
{
	if (!MediaOutputs.IsValidIndex(CurrentIndex) || !MediaOutputs.IsValidIndex(DestIndex))
	{
		return false;
	}

	PlaybackManager->ChangeMediaOutputIndex(CurrentIndex, DestIndex);
	TObjectPtr<UMediaOutput> MediaOutput = MediaOutputs[CurrentIndex];
	MediaOutputs.RemoveAt(CurrentIndex, EAllowShrinking::No);
	MediaOutputs.Insert(MediaOutput, DestIndex);

#if WITH_EDITOR
	if (MediaOutputLabels.IsValidIndex(CurrentIndex))
	{
		FString OutputLabel = MediaOutputLabels[CurrentIndex];
		MediaOutputLabels.RemoveAt(CurrentIndex, EAllowShrinking::No);
		MediaOutputLabels.Insert(OutputLabel, DestIndex);
	}
#endif
	
	return true;
}

#if WITH_EDITOR
FString UMediaProfile::GetLabelForMediaOutput(int32 Index)
{
	if (!MediaOutputLabels.IsValidIndex(Index))
	{
		if (MediaOutputs.IsValidIndex(Index))
		{
			if (MediaOutputs[Index])
			{
				return MediaOutputs[Index]->GetName();
			}
			else
			{
				return TEXT("");
			}
		}
		
		return TEXT("");
	}
	
	return MediaOutputLabels[Index];
}

void UMediaProfile::SetLabelForMediaOutput(int32 Index, const FString& NewLabel)
{
	if (!MediaOutputLabels.IsValidIndex(Index))
	{
		if (!MediaOutputs.IsValidIndex(Index))
		{
			return;
		}
		
		// In some cases, we may not have labels for all media outputs yet (such as an older media profile being edited)
		// In that case, add all the missing labels now
		for (int32 MissingIdx = MediaOutputLabels.Num(); MissingIdx < MediaOutputs.Num(); ++MissingIdx)
		{
			if (MediaOutputs[MissingIdx])
			{
				MediaOutputLabels.Add(MediaOutputs[MissingIdx]->GetName());
			}
			else
			{
				MediaOutputLabels.Add(TEXT(""));
			}
		}
	}
	
	MediaOutputLabels[Index] = UE::MediaProfile::Private::GetUniqueName(NewLabel, MediaOutputLabels);

	FPropertyChangedEvent PropertyChangedEvent(StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMediaProfile, MediaOutputLabels)), EPropertyChangeType::ValueSet);
	FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(this, PropertyChangedEvent);
}
#endif //WITH_EDITOR

UTimecodeProvider* UMediaProfile::GetTimecodeProvider() const
{
	return bOverrideTimecodeProvider ? TimecodeProvider : nullptr;
}


UEngineCustomTimeStep* UMediaProfile::GetCustomTimeStep() const
{
	return bOverrideCustomTimeStep ? CustomTimeStep : nullptr;
}

void UMediaProfile::ApplyTimecodeProvider()
{
	ResetTimecodeProvider();
	if (bOverrideTimecodeProvider)
	{
		bTimecodeProvideWasApplied = true;
		AppliedTimecodeProvider = TimecodeProvider;
		PreviousTimecodeProvider = GEngine->GetTimecodeProvider();

		if (PreviousTimecodeProvider && AppliedTimecodeProvider && PreviousTimecodeProvider->GetClass() == AppliedTimecodeProvider->GetClass())
		{
			GEngine->SetTimecodeProvider(nullptr);
			// Sleep to let the timecode provider complete its shutdown on its hardware.
			FPlatformProcess::Sleep(0.2f);
		}

		bool bResult = GEngine->SetTimecodeProvider(TimecodeProvider);
		if (!bResult && TimecodeProvider)
		{
			UE_LOGF(LogMediaProfile, Warning, "The Timecode Provider '%ls' could not be initialized.", *TimecodeProvider->GetName());
		}
	}
}

void UMediaProfile::ApplyCustomTimeStep()
{
	ResetCustomTimeStep();
	if (bOverrideCustomTimeStep)
	{
		bCustomTimeStepWasApplied = true;
		AppliedCustomTimeStep = CustomTimeStep;
		PreviousCustomTimeStep = GEngine->GetCustomTimeStep();
		bool bResult = GEngine->SetCustomTimeStep(CustomTimeStep);
		if (!bResult && CustomTimeStep)
		{
			UE_LOGF(LogMediaProfile, Warning, "The Custom Time Step '%ls' could not be initialized.", *CustomTimeStep->GetName());
		}
	}
}

void UMediaProfile::Apply()
{
#if WITH_EDITORONLY_DATA
	bNeedToBeReapplied = false;
#endif

	if (GEngine == nullptr)
	{
		UE_LOGF(LogMediaProfile, Error, "The MediaProfile '%ls' could not be applied. The Engine is not initialized.", *GetName());
		return;
	}

	// Make sure we have the same amount of sources and outputs as the number of proxies.
	FixNumSourcesAndOutputs();

	{
		TArray<UProxyMediaSource*> SourceProxies = IMediaProfileManager::Get().GetAllMediaSourceProxy();
		check(SourceProxies.Num() == MediaSources.Num());
		for (int32 Index = 0; Index < MediaSources.Num(); ++Index)
		{
			UProxyMediaSource* Proxy = SourceProxies[Index];
			if (Proxy)
			{
				Proxy->SetDynamicMediaSource(MediaSources[Index]);
			}
		}
	}

	{
		TArray<UProxyMediaOutput*> OutputProxies = IMediaProfileManager::Get().GetAllMediaOutputProxy();
		check(OutputProxies.Num() == MediaOutputs.Num());
		for (int32 Index = 0; Index < MediaOutputs.Num(); ++Index)
		{
			UProxyMediaOutput* Proxy = OutputProxies[Index];
			if (Proxy)
			{
				Proxy->SetDynamicMediaOutput(MediaOutputs[Index]);
			}
		}
	}

	ApplyTimecodeProvider();
	ApplyCustomTimeStep();

	SendAnalytics();
}


void UMediaProfile::Reset()
{
	if (GEngine == nullptr)
	{
		UE_LOGF(LogMediaProfile, Error, "The MediaProfile '%ls' could not be reset. The Engine is not initialized.", *GetName());
		return;
	}

	{
		// Reset the source proxies
		TArray<UProxyMediaSource*> SourceProxies = IMediaProfileManager::Get().GetAllMediaSourceProxy();
		for (UProxyMediaSource* Proxy : SourceProxies)
		{
			if (Proxy)
			{
				Proxy->SetDynamicMediaSource(nullptr);
			}
		}
	}

	{
		// Reset the output proxies
		TArray<UProxyMediaOutput*> OutputProxies = IMediaProfileManager::Get().GetAllMediaOutputProxy();
		for (UProxyMediaOutput* Proxy : OutputProxies)
		{
			if (Proxy)
			{
				Proxy->SetDynamicMediaOutput(nullptr);
			}
		}
	}

	// Reset the timecode provider
	ResetTimecodeProvider();

	// Reset the engine custom time step
	ResetCustomTimeStep();
}

void UMediaProfile::ResetTimecodeProvider()
{
	if (!bTimecodeProvideWasApplied)
	{
		return;
	}

	// Optional but recommended in UE code that touches engine-wide state
	ensure(IsInGameThread());

	if (!GEngine)
	{
		UE_LOGF(LogMediaProfile, Warning, "Cannot reset TimecodeProvider: GEngine is null.");
		return; // keep state so caller can retry later
	}

	auto GetNameSafe = [](const UObject* Obj) -> FString
		{
			return (Obj && IsValid(Obj)) ? Obj->GetName() : TEXT("None");
		};

	UTimecodeProvider* const CurrentProvider = GEngine->GetTimecodeProvider();

	// Only restore if we still own the setting (don’t clobber external changes)
	const bool bAppliedValid = (AppliedTimecodeProvider && IsValid(AppliedTimecodeProvider));
	const bool bWeStillOwn = (bAppliedValid && AppliedTimecodeProvider == CurrentProvider);

	bool bRestored = false;

	if (bWeStillOwn)
	{
		// If previous is invalid, reset to null before attempting to apply
		if (PreviousTimecodeProvider && !IsValid(PreviousTimecodeProvider))
		{
			UE_LOGF(LogMediaProfile, Warning, "Previous TimecodeProvider is no longer valid. Resetting to null.");
			PreviousTimecodeProvider = nullptr;
		}

		bRestored = GEngine->SetTimecodeProvider(PreviousTimecodeProvider);

		if (!bRestored)
		{
			if (PreviousTimecodeProvider && IsValid(PreviousTimecodeProvider))
			{
				UE_LOGF(LogMediaProfile, Warning,
					"The TimecodeProvider '%ls' could not be initialized.",
					*PreviousTimecodeProvider->GetName());
			}
			else
			{
				UE_LOGF(LogMediaProfile, Warning,
					"Could not clear the TimecodeProvider.");
			}
		}
	}
	else
	{
		// Clarify whether it changed externally or our applied instance is gone
		if (!bAppliedValid && AppliedTimecodeProvider)
		{
			UE_LOGF(LogMediaProfile, Warning,
				"Previously applied TimecodeProvider is no longer valid. Not restoring.");
		}
		else
		{
			UE_LOGF(LogMediaProfile, Warning,
				"TimecodeProvider changed externally (current: '%ls', previously applied: '%ls'). Not restoring.",
				*GetNameSafe(CurrentProvider), *GetNameSafe(AppliedTimecodeProvider));
		}
	}

	// Only clear state if we either restored successfully or we explicitly decided not to restore
	if (bRestored || !bWeStillOwn)
	{
		PreviousTimecodeProvider = nullptr;
		AppliedTimecodeProvider = nullptr;
		bTimecodeProvideWasApplied = false;
	}
}


void UMediaProfile::ResetCustomTimeStep()
{
	if (!bCustomTimeStepWasApplied)
	{
		return;
	}

	// Recommended: enforce game-thread usage when touching engine-wide state
	ensure(IsInGameThread());

	if (!GEngine)
	{
		UE_LOGF(LogMediaProfile, Warning, "Cannot reset CustomTimeStep: GEngine is null.");
		return; // keep state so we can retry later
	}

	auto GetNameSafe = [](const UObject* Obj) -> FString
		{
			return (Obj && IsValid(Obj)) ? Obj->GetName() : TEXT("None");
		};

	// Ownership check: only restore if the engine still has what we applied
	UEngineCustomTimeStep* const CurrentCTS = GEngine->GetCustomTimeStep();
	const bool bAppliedValid = (AppliedCustomTimeStep && IsValid(AppliedCustomTimeStep));
	const bool bWeStillOwn = (bAppliedValid && AppliedCustomTimeStep == CurrentCTS);

	bool bRestored = false;

	if (bWeStillOwn)
	{
		// If previous is invalid, reset to null to avoid passing a dead object
		if (PreviousCustomTimeStep && !IsValid(PreviousCustomTimeStep))
		{
			UE_LOGF(LogMediaProfile, Warning,
				"Previous Custom Time Step is no longer valid. Resetting to null.");
			PreviousCustomTimeStep = nullptr;
		}

		bRestored = GEngine->SetCustomTimeStep(PreviousCustomTimeStep);

		if (!bRestored)
		{
			if (PreviousCustomTimeStep && IsValid(PreviousCustomTimeStep))
			{
				UE_LOGF(LogMediaProfile, Warning,
					"The Custom Time Step '%ls' could not be initialized.",
					*PreviousCustomTimeStep->GetName());
			}
			else
			{
				UE_LOGF(LogMediaProfile, Warning,
					"Could not clear the Custom Time Step.");
			}
		}
	}
	else
	{
		// Clarify why we’re not restoring
		if (!bAppliedValid && AppliedCustomTimeStep)
		{
			UE_LOGF(LogMediaProfile, Warning,
				"Previously applied Custom Time Step is no longer valid. Not restoring.");
		}
		else
		{
			UE_LOGF(LogMediaProfile, Warning,
				"Custom Time Step changed externally (current: '%ls', previously applied: '%ls'). Not restoring.",
				*GetNameSafe(CurrentCTS), *GetNameSafe(AppliedCustomTimeStep));
		}
	}

	// Only clear our local state if we either restored successfully or explicitly chose not to restore
	if (bRestored || !bWeStillOwn)
	{
		PreviousCustomTimeStep = nullptr;
		AppliedCustomTimeStep = nullptr;
		bCustomTimeStepWasApplied = false;
	}
}


/**
 * @EventName MediaFramework.ApplyMediaProfile
 * @Trigger Triggered when a media profile is applied.
 * @Type Client
 * @Owner MediaIO Team
 */
void UMediaProfile::SendAnalytics() const
{
#if WITH_EDITOR
	if (FEngineAnalytics::IsAvailable())
	{
		const FString TimecodeProviderName = GetTimecodeProvider() ? GetTimecodeProvider()->GetName() : TEXT("None");
		const FString CustomTimestepName = GetCustomTimeStep() ? GetCustomTimeStep()->GetName() : TEXT("None");
		
		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Inputs"), MediaProfileAnalytics::JoinObjectNames<UMediaSource>(MediaSources)));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Outputs"), MediaProfileAnalytics::JoinObjectNames<UMediaOutput>(MediaOutputs)));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("TimecodeProvider"), TimecodeProviderName));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("CustomTimeStep"), CustomTimestepName));
		
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("MediaFramework.ApplyMediaProfile"), EventAttributes);
	}
#endif
}

void UMediaProfile::FixNumSourcesAndOutputs()
{
	const int32 NumSourceProxies = IMediaProfileManager::Get().GetAllMediaSourceProxy().Num();
	const int32 NumOutputProxies = IMediaProfileManager::Get().GetAllMediaOutputProxy().Num();

	const bool bResizeMediaSources = MediaSources.Num() != NumSourceProxies;
	const bool bResizeMediaOutputs = MediaOutputs.Num() != NumOutputProxies;

	if (bResizeMediaSources || bResizeMediaOutputs)
	{
		Modify();
	}
	
	if (bResizeMediaSources)
	{
		MediaSources.SetNumZeroed(NumSourceProxies);

#if WITH_EDITOR
		FPropertyChangedEvent PropertyChangedEvent(StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMediaProfile, MediaSources)), EPropertyChangeType::ArrayAdd);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(this, PropertyChangedEvent);
#endif // WITH_EDITOR
	}

	if (bResizeMediaOutputs)
	{
		MediaOutputs.SetNumZeroed(NumOutputProxies);

#if WITH_EDITOR
		FPropertyChangedEvent PropertyChangedEvent(StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMediaProfile, MediaOutputs)), EPropertyChangeType::ArrayAdd);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(this, PropertyChangedEvent);
#endif // WITH_EDITOR
	}
}

UMediaProfilePlaybackManager* UMediaProfile::GetPlaybackManager() const
{
	return PlaybackManager;
}
