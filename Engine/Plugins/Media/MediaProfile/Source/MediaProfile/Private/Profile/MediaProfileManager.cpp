// Copyright Epic Games, Inc. All Rights Reserved.
#include "Profile/MediaProfileManager.h"
#include "MediaProfileModule.h"

#include "Engine/Engine.h"
#include "Modules/ModuleManager.h"
#include "MediaAssets/ProxyMediaOutput.h"
#include "MediaAssets/ProxyMediaSource.h"
#include "Misc/App.h"
#include "Profile/MediaProfile.h"
#include "Profile/MediaProfileSettings.h"


IMediaProfileManager& IMediaProfileManager::Get()
{
	static FName NAME_MediaProfileModule = TEXT("MediaProfile");
	return FModuleManager::GetModuleChecked<IMediaProfileModule>(NAME_MediaProfileModule).GetProfileManager();
}


FMediaProfileManager::FMediaProfileManager()
	: CurrentMediaProfile(nullptr)
{
	if (UObjectInitialized())
	{
		if (FApp::CanEverRender() && GEngine && GEngine->IsInitialized())
		{
			MediaSourceProxies = ObjectPtrWrap(GetDefault<UMediaProfileSettings>()->LoadMediaSourceProxies());
			MediaOutputProxies = ObjectPtrWrap(GetDefault<UMediaProfileSettings>()->LoadMediaOutputProxies());
		}

#if WITH_EDITOR
		GetMutableDefault<UMediaProfileSettings>()->OnMediaProxiesChanged.AddRaw(this, &FMediaProfileManager::OnMediaProxiesChanged);
#endif
	}
}


FMediaProfileManager::~FMediaProfileManager()
{
#if WITH_EDITOR
	if (UObjectInitialized())
	{
		GetMutableDefault<UMediaProfileSettings>()->OnMediaProxiesChanged.RemoveAll(this);
	}
#endif
}


void FMediaProfileManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(CurrentMediaProfile);
	Collector.AddReferencedObjects(MediaSourceProxies);
	Collector.AddReferencedObjects(MediaOutputProxies);
}


UMediaProfile* FMediaProfileManager::GetCurrentMediaProfile() const
{
	return CurrentMediaProfile;
}


TArray<UProxyMediaSource*> FMediaProfileManager::GetAllMediaSourceProxy() const
{
	return ObjectPtrDecay(MediaSourceProxies);
}


TArray<UProxyMediaOutput*> FMediaProfileManager::GetAllMediaOutputProxy() const
{
	return ObjectPtrDecay(MediaOutputProxies);
}


void FMediaProfileManager::SetCurrentMediaProfile(UMediaProfile* InMediaProfile)
{
	MediaSourceProxies = ObjectPtrWrap(GetDefault<UMediaProfileSettings>()->LoadMediaSourceProxies());
	MediaOutputProxies = ObjectPtrWrap(GetDefault<UMediaProfileSettings>()->LoadMediaOutputProxies());

	UMediaProfile* Previous = CurrentMediaProfile;
	if (InMediaProfile != Previous)
	{
		if (Previous)
		{
			Previous->Reset();
		}

		if (InMediaProfile)
		{
			InMediaProfile->Apply();
		}

		CurrentMediaProfile = InMediaProfile;
		MediaProfileChangedDelegate.Broadcast(Previous, InMediaProfile);
	}
}


FMediaProfileManager::FOnMediaProfileChanged& FMediaProfileManager::OnMediaProfileChanged()
{
	return MediaProfileChangedDelegate;
}


#if WITH_EDITOR
void FMediaProfileManager::OnMediaProxiesChanged()
{
	MediaSourceProxies = ObjectPtrWrap(GetDefault<UMediaProfileSettings>()->LoadMediaSourceProxies());
	MediaOutputProxies = ObjectPtrWrap(GetDefault<UMediaProfileSettings>()->LoadMediaOutputProxies());

	if (CurrentMediaProfile)
	{
		CurrentMediaProfile->Reset();
		CurrentMediaProfile->Apply();
	}
}
#endif
