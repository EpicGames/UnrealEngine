// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Profile/IMediaProfileManager.h"
#include "UObject/GCObject.h"

class UMediaProfile;
class UProxyMediaSource;
class UProxyMediaOutput;

class FMediaProfileManager : public IMediaProfileManager, public FGCObject
{
public:
	FMediaProfileManager();
	virtual ~FMediaProfileManager();

	//~ Begin IMediaProfileManager Interface
	virtual UMediaProfile* GetCurrentMediaProfile() const override;
	virtual void SetCurrentMediaProfile(UMediaProfile* InMediaProfile) override;
	TArray<UProxyMediaSource*> GetAllMediaSourceProxy() const override;
	TArray<UProxyMediaOutput*> GetAllMediaOutputProxy() const override;
	virtual FOnMediaProfileChanged& OnMediaProfileChanged() override;
	//~ End IMediaProfileManager Interface

	//~ Begin FGCObject Interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FMediaProfileManager");
	}
	//~End FGCObject Interface

private:
#if WITH_EDITOR
	void OnMediaProxiesChanged();
#endif

	TArray<TObjectPtr<UProxyMediaSource>> MediaSourceProxies;
	TArray<TObjectPtr<UProxyMediaOutput>> MediaOutputProxies;
	TObjectPtr<UMediaProfile> CurrentMediaProfile;
	FOnMediaProfileChanged MediaProfileChangedDelegate;
};
