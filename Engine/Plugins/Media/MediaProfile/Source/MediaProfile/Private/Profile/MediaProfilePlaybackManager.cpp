// Copyright Epic Games, Inc. All Rights Reserved.


#include "Profile/MediaProfilePlaybackManager.h"

#include "IMediaProfileModule.h"
#include "Engine/TextureRenderTarget2D.h"
#include "MediaCapture.h"
#include "MediaOutput.h"
#include "MediaPlayer.h"
#include "MediaPlayerFacade.h"
#include "MediaSource.h"
#include "MediaTexture.h"
#include "TimerManager.h"
#include "ViewportClient.h"
#include "Engine/Engine.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "Profile/MediaProfile.h"
#include "Slate/SceneViewport.h"

#if WITH_EDITOR
#include "EditorViewportClient.h"
#include "Framework/Application/SlateApplication.h"
#include "IAssetViewport.h"
#include "LevelEditor.h"
#include "LevelEditorViewport.h"
#include "Widgets/SViewport.h"
#endif

namespace MediaProfilePlaybackManager
{
	const FString DefaultMediaTexturesPath = TEXT("/MediaFrameworkUtilities/DefaultMediaTextures");
	const FString DefaultMediaTextureName = TEXT("MediaProfileTexture");

	/** Common interface for all types of media capture viewport clients */
	class IMediaCaptureViewportClient : public TSharedFromThis<IMediaCaptureViewportClient>
	{
	public:
		virtual ~IMediaCaptureViewportClient() = default;
	
		virtual void SetPIE(bool bInIsPIE) = 0;
		virtual TSharedRef<FViewportClient> AsViewportClient() = 0;
	};
	
#if WITH_EDITOR
	class FMediaCaptureEditorViewportClient : public FLevelEditorViewportClient, public IMediaCaptureViewportClient
	{
	private:
		using Super = FLevelEditorViewportClient;

	public:
		FMediaCaptureEditorViewportClient(EViewModeIndex InViewModeIndex)
			: FLevelEditorViewportClient(nullptr)
			, ViewModeIndex(InViewModeIndex)
		{
			bSetListenerPosition = false;

			// Default to "game" show flags for camera previews
			EngineShowFlags = FEngineShowFlags(ESFIM_Game);
			LastEngineShowFlags = FEngineShowFlags(ESFIM_Editor);

			ViewportType = LVT_Perspective;
			bDrawAxes = false;
			bDisableInput = true;
			SetAllowCinematicControl(false);
			VisibilityDelegate.BindLambda([] { return true; });
			AutoSetPIE();
		}

		virtual ~FMediaCaptureEditorViewportClient() override
		{
			if (GEditor)
			{
				FWorldContext* PIEWorldContext = GEditor->GetPIEWorldContext();
				if (PIEWorldContext)
				{
					RemoveReferenceToWorldContext(*PIEWorldContext);
				}
			}
		}
		
		virtual UWorld* GetWorld() const override
		{
			if (GEditor->PlayWorld)
			{
				return GEditor->PlayWorld;
			}
			return Super::GetWorld();
		}

		virtual void SetPIE(bool bInIsPIE) override
		{
			FWorldContext* PIEWorldContext = GEditor->GetPIEWorldContext();
			if (bInIsPIE)
			{
				if (PIEWorldContext)
				{
					RemoveReferenceToWorldContext(GEditor->GetEditorWorldContext());
					SetReferenceToWorldContext(*PIEWorldContext);
				}
			}
			else
			{
				if (PIEWorldContext)
				{
					RemoveReferenceToWorldContext(*PIEWorldContext);
				}
				SetReferenceToWorldContext(GEditor->GetEditorWorldContext());
			}
			SetViewMode(ViewModeIndex);
			SetRealtime(true);
		}

		void AutoSetPIE()
		{
			if (GEditor->PlayWorld)
			{
				SetPIE(true);
			}
			else
			{
				SetViewMode(ViewModeIndex);
				SetRealtime(true);
			}
		}

		virtual TSharedRef<FViewportClient> AsViewportClient() override
		{
			return SharedThis(this);
		}

	private:
		EViewModeIndex ViewModeIndex;
	};
#endif

	// TODO: If media capture in a Game build ever becomes supported, this should be filled out as appropriate
	class FMediaCaptureGameViewportClient : public FCommonViewportClient, public IMediaCaptureViewportClient
	{
		virtual void SetPIE(bool bInIsPIE) override { }
		virtual TSharedRef<FViewportClient> AsViewportClient() override { return SharedThis(this); }
	};

	/** Stores and manages all the necessary objects for a managed capture viewport */
	class FManagedViewport
	{
	public:
		/** The media output the managed viewport is for */
		TWeakObjectPtr<UMediaOutput> MediaOutput;

		/** The viewport client of the viewport */
		TSharedPtr<IMediaCaptureViewportClient> ViewportClient;

		/** The viewport being managed */
		TSharedPtr<FSceneViewport> Viewport;

#if WITH_EDITOR
		/** Scene viewports needs a viewport widget to properly function when in editor */
		TSharedPtr<SViewport> ViewportWidget;
#endif

		/** A list of consumers that are actively using the managed viewport */
		TSet<void*> Consumers;
		
		FManagedViewport(UMediaOutput* InMediaOutput)
			: MediaOutput(InMediaOutput)
		{ }
	};

	/** Creates a new managed viewport for capturing */
	TSharedPtr<FManagedViewport> CreateManagedViewport(UMediaOutput* InMediaOutput, EViewModeIndex InViewMode)
	{
		TSharedPtr<FManagedViewport> ManagedViewport = MakeShared<FManagedViewport>(InMediaOutput);
		
#if WITH_EDITOR
		IMediaProfileModule& MediaProfileModule = FModuleManager::GetModuleChecked<IMediaProfileModule>("MediaProfile");
		if (MediaProfileModule.CanCaptureWithLevelEditorViewportClients())
		{
			TSharedPtr<FMediaCaptureEditorViewportClient> ViewportClient = MakeShared<FMediaCaptureEditorViewportClient>(InViewMode);
		
			ManagedViewport->ViewportClient = ViewportClient;
			ManagedViewport->ViewportWidget = SNew(SViewport)
				.RenderDirectlyToWindow(false)
				.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
				.EnableGammaCorrection(false)
				.EnableBlending(false);

			ManagedViewport->Viewport = FSceneViewport::Create(ViewportClient, ManagedViewport->ViewportWidget);

			ViewportClient->Viewport = ManagedViewport->Viewport.Get();
		}
		else
#endif
		{
			TSharedPtr<FMediaCaptureGameViewportClient> ViewportClient = MakeShared<FMediaCaptureGameViewportClient>();

			ManagedViewport->ViewportClient = ViewportClient;
			ManagedViewport->Viewport = FSceneViewport::Create(ViewportClient, nullptr);
		}

		return ManagedViewport;
	}

	/** Searches for an active scene viewport in the engine */
	TSharedPtr<FSceneViewport> FindActiveViewport()
	{
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
#if WITH_EDITOR
			// If there is an active PIE viewport, return it
			if (Context.WorldType == EWorldType::PIE)
			{
				UEditorEngine* EditorEngine = CastChecked<UEditorEngine>(GEngine);
				FSlatePlayInEditorInfo& Info = EditorEngine->SlatePlayInEditorMap.FindChecked(Context.ContextHandle);
				if (Info.SlatePlayInEditorWindowViewport.IsValid())
				{
					return Info.SlatePlayInEditorWindowViewport;
				}
			}
#endif

			// TODO: If media capture in a Game build ever becomes supported, an appropriate game viewport should be found and returned
		}

		return nullptr;
	}

#if WITH_EDITOR
	/** Searches for a level editor viewport */
	TSharedPtr<IAssetViewport> FindActiveViewportInterface()
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::Get().GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		return LevelEditorModule.GetFirstActiveViewport();
	}
#endif

	/** Stores references to an existing scene viewport managed by the engine or editor, and stores cached view state for that viewport */
	class FActiveViewport
	{
	public:
		FActiveViewport(UMediaOutput* InMediaOutput, const FMediaCaptureOptions& InCaptureOptions)
			: MediaOutput(InMediaOutput)
		{
			Viewport = FindActiveViewport();
			
#if WITH_EDITOR
			if (!Viewport.IsValid())
			{
				if (TSharedPtr<IAssetViewport> ActiveViewportInterface = FindActiveViewportInterface())
				{
					ViewportInterface = ActiveViewportInterface;
				
					FEditorViewportClient& ViewportClient = ActiveViewportInterface->GetAssetViewportClient();

					// Cache the viewport's current settings, then set them to more appropriate settings for recording
					CachedViewportFlags.bRealTime = ViewportClient.IsRealtime();
					CachedViewportFlags.bSetListenerPosition = ViewportClient.bSetListenerPosition;
					CachedViewportFlags.bDrawAxes = ViewportClient.bDrawAxes;
					CachedViewportFlags.bAllowCinematicControl = ViewportClient.AllowsCinematicControl();
					CachedViewportFlags.bDrawAlphaBlendedCheckerboard = ViewportClient.ChannelMaskParams.bDrawAlphaBlendedCheckerboard;
					CachedViewportFlags.VisibilityDelegate = ViewportClient.VisibilityDelegate;

					ViewportClient.SetRealtime(true);
					ViewportClient.bSetListenerPosition = false;
					ViewportClient.bDrawAxes = false;
					ViewportClient.SetAllowCinematicControl(false);
					ViewportClient.ChannelMaskParams.bDrawAlphaBlendedCheckerboard = false;
					ViewportClient.VisibilityDelegate.BindLambda([] { return true; });

					FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
					LevelEditorModule.OnGetVisualizeCheckerboardEnabled().BindLambda([] { return false; });

					if (InCaptureOptions.ResizeMethod == EMediaCaptureResizeMethod::ResizeSource)
					{
						if (TSharedPtr<FSceneViewport> LevelViewport = ActiveViewportInterface->GetSharedActiveViewport())
						{
							if (LevelViewport->HasFixedSize())
							{
								CachedViewportFlags.ViewportSize = LevelViewport->GetSize();
							}
						
							FIntPoint TargetSize = MediaOutput->GetRequestedSize();
							if (TargetSize == UMediaOutput::RequestCaptureSourceSize)
							{
								TargetSize = FIntPoint(1920, 1080);
							}
						
							LevelViewport->SetFixedViewportSize(TargetSize.X, TargetSize.Y);
							LevelViewport->InvalidateHitProxy();
						}
					}
				}
			}
#endif
		}

		~FActiveViewport()
		{
#if WITH_EDITOR
			if (TSharedPtr<IAssetViewport> PinnedViewportInterface = ViewportInterface.Pin())
			{
				FEditorViewportClient& ViewportClient = PinnedViewportInterface->GetAssetViewportClient();

				// Reset settings
				ViewportClient.SetRealtime(CachedViewportFlags.bRealTime);
				ViewportClient.bSetListenerPosition = CachedViewportFlags.bSetListenerPosition;
				ViewportClient.bDrawAxes = CachedViewportFlags.bDrawAxes;
				ViewportClient.SetAllowCinematicControl(CachedViewportFlags.bAllowCinematicControl);
				ViewportClient.ChannelMaskParams.bDrawAlphaBlendedCheckerboard = CachedViewportFlags.bDrawAlphaBlendedCheckerboard;
				ViewportClient.VisibilityDelegate = CachedViewportFlags.VisibilityDelegate;

				FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
				LevelEditorModule.OnGetVisualizeCheckerboardEnabled().Unbind();
				
				if (TSharedPtr<FSceneViewport> LevelViewport = PinnedViewportInterface->GetSharedActiveViewport())
				{
					LevelViewport->SetFixedViewportSize(CachedViewportFlags.ViewportSize.X, CachedViewportFlags.ViewportSize.Y);
					LevelViewport->InvalidateHitProxy();
				}
			}
#endif
		}

		TSharedPtr<FSceneViewport> GetSceneViewport() const
		{
			if (TSharedPtr<FSceneViewport> PinnedActiveViewport = Viewport.Pin())
			{
				return PinnedActiveViewport;
			}

#if WITH_EDITOR
			if (TSharedPtr<IAssetViewport> PinnedViewportInterface = ViewportInterface.Pin())
			{
				return PinnedViewportInterface->GetSharedActiveViewport();
			}
#endif

			return nullptr;
		}
		
		/** The media output that is using this active viewport */
		TWeakObjectPtr<UMediaOutput> MediaOutput;

		/** Weak reference to an existing scene viewport, if one was found */
		TWeakPtr<FSceneViewport> Viewport;

#if WITH_EDITOR
		/** Weak reference to a level editor viewport, if one was found */
		TWeakPtr<IAssetViewport> ViewportInterface;

		/** Cached view state of the active viewport client, to be re-instated once the viewport is no longer being captured */
		struct
		{
			FIntPoint ViewportSize = FIntPoint::ZeroValue;
			bool bRealTime = false;
			bool bSetListenerPosition = false;
			bool bDrawAxes = false;
			bool bAllowCinematicControl = false;
			bool bDrawAlphaBlendedCheckerboard = false;
			FEngineShowFlags EngineShowFlags = ESFIM_Editor;
			FEngineShowFlags LastEngineShowFlags = ESFIM_Game;
			FViewportStateGetter VisibilityDelegate;
		} CachedViewportFlags;
#endif
	};

	/** Keeps track of what media textures are being captured by media outputs in the media profile to help determine when the captures should be automatically restarted on changes */
	class FMediaTextureCapture
	{
	public:
		FMediaTextureCapture(UMediaOutput* InMediaOutput)
			: MediaOutput(InMediaOutput)
		{ }

		/** Updates the media texture and capture options of the media texture capture */
		void UpdateMediaTexture(UMediaTexture* InMediaTexture, const FMediaCaptureOptions& InCaptureOptions, const FMediaCaptureTransform& InCaptureTransform, bool bInAutoRestartCapture)
		{
			MediaTexture = InMediaTexture;
			CaptureOptions = InCaptureOptions;
			CaptureTransform = InCaptureTransform;
			bAutoRestartCapture = bInAutoRestartCapture;
			
			if (MediaTexture.IsValid())
			{
				LastMediaTextureSize = FIntPoint(MediaTexture->GetWidth(), MediaTexture->GetHeight());
			}
		}
		
		/** Checks whether the media texture has changed size since the last check */
		bool NeedsCaptureRestart()
		{
			if (!MediaOutput.IsValid() || !MediaTexture.IsValid())
			{
				return false;
			}
			
			const FIntPoint MediaTextureSize = FIntPoint(MediaTexture->GetWidth(), MediaTexture->GetHeight());
			const bool bTextureSizeChanged = MediaTextureSize != LastMediaTextureSize;
			LastMediaTextureSize = MediaTextureSize;
			return bTextureSizeChanged && bAutoRestartCapture;
		}
		
		/** The media output that is capturing the media texture */
		TWeakObjectPtr<UMediaOutput> MediaOutput = nullptr;
		
		/** The media texture being captured */
		TWeakObjectPtr<UMediaTexture> MediaTexture = nullptr;
		
		/** The options the media texture capture was initialized with */
		FMediaCaptureOptions CaptureOptions;
		
		/** The transform the media texture capture was initialized with */
		FMediaCaptureTransform CaptureTransform;
		
		/** Whether the media texture capture can auto-restart when a change is detected */
		bool bAutoRestartCapture = false;
		
		/** Keeps track of the last media texture size to recognize changes in media texture resolution that indicate a capture restart is needed */
		FIntPoint LastMediaTextureSize = FIntPoint::ZeroValue;
	};
}

bool MediaProfilePlaybackManager::IsActiveViewportCaptureAllowed()
{
	static TOptional<bool> bAllowActiveViewportCapture;
	if (!bAllowActiveViewportCapture.IsSet())
	{
		// If the config param does not exist in the ini, default to true
		bool bConfigValue = true;
		GConfig->GetBool(TEXT("MediaProfile"), TEXT("bAllowActiveViewportCapture"), bConfigValue, GEngineIni);
		bAllowActiveViewportCapture = bConfigValue;
	}
	return bAllowActiveViewportCapture.Get(true);
}

bool MediaProfilePlaybackManager::IsManagedViewportCaptureAllowed()
{
	static TOptional<bool> bAllowManagedViewportCapture;
	if (!bAllowManagedViewportCapture.IsSet())
	{
		// If the config param does not exist in the ini, default to true
		bool bConfigValue = true;
		GConfig->GetBool(TEXT("MediaProfile"), TEXT("bAllowManagedViewportCapture"), bConfigValue, GEngineIni);
		bAllowManagedViewportCapture = bConfigValue;
	}
	return bAllowManagedViewportCapture.Get(true);
}

UMediaProfilePlaybackManager::UMediaProfilePlaybackManager(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	constexpr int32 NumMediaTextures = 16;
	for (int32 TextureIndex = 1; TextureIndex <= NumMediaTextures; ++TextureIndex)
	{
		const FString IndexStr = TextureIndex > 9 ? FString::FromInt(TextureIndex) : TEXT("0") + FString::FromInt(TextureIndex);
		const FString TextureAssetName = FString::Format(TEXT("{0}_{1}"), {
			MediaProfilePlaybackManager::DefaultMediaTextureName,
			IndexStr });
		
		const FString Path = FString::Format(TEXT("{0}/{1}.{1}"), {
			MediaProfilePlaybackManager::DefaultMediaTexturesPath,
			TextureAssetName });

		MediaSourceTextures.Add(TSoftObjectPtr<UMediaTexture>(FSoftObjectPath(Path)));
	}

	// Close all open sources and outputs when the engine shuts down. If we are in an editor, perform the cleanup a bit earlier
	// than the Exit process to avoid issues with live viewport clients during the editor cleanup process
#if WITH_EDITOR
	if (GEditor && !IsTemplate())
	{
		GEditor->OnEditorClose().AddUObject(this, &UMediaProfilePlaybackManager::Cleanup);
	}
#else
	FCoreDelegates::OnEnginePreExit.AddUObject(this, &UMediaProfilePlaybackManager::Cleanup);
#endif
}

UMediaProfilePlaybackManager::~UMediaProfilePlaybackManager()
{
#if WITH_EDITOR
	if (GEditor && !IsTemplate())
	{
		GEditor->OnEditorClose().RemoveAll(this);
	}
#else
	FCoreDelegates::OnEnginePreExit.RemoveAll(this);
#endif
}

void UMediaProfilePlaybackManager::Tick(float DeltaTime)
{
	using namespace MediaProfilePlaybackManager;
	
	TArray<TSharedPtr<FMediaTextureCapture>> ChangedMediaTextures;
	for (TSharedPtr<FMediaTextureCapture>& MediaTextureCapture : MediaTextureCaptures)
	{
		if (!MediaTextureCapture.IsValid())
		{
			continue;
		}
		
		if (MediaTextureCapture->NeedsCaptureRestart())
		{
			ChangedMediaTextures.Add(MediaTextureCapture);
		}
	}
	
	for (TSharedPtr<FMediaTextureCapture>& ChangedMediaTexture : ChangedMediaTextures)
	{
		RestartMediaTextureOutput(
			ChangedMediaTexture->MediaOutput.Get(),
			ChangedMediaTexture->MediaTexture.Get(), 
			ChangedMediaTexture->CaptureOptions,
			ChangedMediaTexture->CaptureTransform,
			ChangedMediaTexture->bAutoRestartCapture);
	}
}

ETickableTickType UMediaProfilePlaybackManager::GetTickableTickType() const
{
	return HasAnyFlags(RF_ClassDefaultObject) ? ETickableTickType::Never : ETickableTickType::Conditional;
}

bool UMediaProfilePlaybackManager::IsTickable() const
{
	// Keep track of whether the media textures being captured have changed size, requiring a capture restart
	// Check every tick if their size has changed, and if so, call Restart on the capture
	return !MediaTextureCaptures.IsEmpty();
}

UWorld* UMediaProfilePlaybackManager::GetTickableGameObjectWorld() const
{
	return GetWorld();
}

TStatId UMediaProfilePlaybackManager::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UMediaProfilePlaybackManager, STATGROUP_Tickables);
}

UMediaTexture* UMediaProfilePlaybackManager::GetSourceMediaTextureFromIndex(int32 InMediaSourceIndex) const
{
	if (!MediaSourceTextures.IsValidIndex(InMediaSourceIndex))
	{
		return nullptr;
	}

	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return nullptr;
	}
	
	if (InMediaSourceIndex >= MediaProfile->NumMediaSources())
	{
		return nullptr;	
	}

	return GIsRunning ? MediaSourceTextures[InMediaSourceIndex].LoadSynchronous() : nullptr;
}

UMediaTexture* UMediaProfilePlaybackManager::GetSourceMediaTexture(UMediaSource* InMediaSource) const
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return nullptr;
	}
	
	int32 MediaSourceIndex = MediaProfile->FindMediaSourceIndex(InMediaSource);
	return GetSourceMediaTextureFromIndex(MediaSourceIndex);
}

bool UMediaProfilePlaybackManager::IsValidSourceMediaTexture(UMediaTexture* InMediaTexture, int32& OutMediaSourceIndex) const
{
	OutMediaSourceIndex = INDEX_NONE;
	
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return false;
	}
	
	int32 MediaSourceIndex;
	if (MediaSourceTextures.Find(InMediaTexture, MediaSourceIndex))
	{
		if (MediaProfile->NumMediaSources() > MediaSourceIndex)
		{
			OutMediaSourceIndex = MediaSourceIndex;
			return true;
		}
	}

	return false;
}

UMediaTexture* UMediaProfilePlaybackManager::OpenSourceFromIndex(int32 InMediaSourceIndex, void* InConsumer)
{
	if (!MediaSourceTextures.IsValidIndex(InMediaSourceIndex))
	{
		return nullptr;
	}
	
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return nullptr;
	}
	
	if (InMediaSourceIndex < 0 || InMediaSourceIndex >= MediaProfile->NumMediaSources())
	{
		return nullptr;
	}
		
	UMediaTexture* MediaTexture = GetSourceMediaTextureFromIndex(InMediaSourceIndex);
	if (!MediaTexture)
	{
		return nullptr;
	}
	
	UMediaSource* MediaSource = MediaProfile->GetMediaSource(InMediaSourceIndex);
	if (!MediaSource)
	{
		return nullptr;
	}

	UMediaPlayer* MediaSourcePlayer = FindOrCreateMediaPlayerForIndex(InMediaSourceIndex);

	MediaTexture->SetMediaPlayer(MediaSourcePlayer);
	
	if (!MediaSourcePlayer->IsClosed())
	{
		// If the media source is already open, simply return the media texture
		if (MediaSourcePlayer->GetUrl() == MediaSource->GetUrl())
		{
			RegisterMediaSourceConsumer(MediaSourcePlayer, InConsumer);
			return MediaTexture;
		}
	}

	FMediaPlayerOptions Options;
	Options.PlayOnOpen = EMediaPlayerOptionBooleanOverride::Enabled;
	Options.Loop = EMediaPlayerOptionBooleanOverride::Enabled;
	
	if (MediaSourcePlayer->OpenSourceWithOptions(MediaSource, Options))
	{
		RegisterMediaSourceConsumer(MediaSourcePlayer, InConsumer);
		return MediaTexture;
	}
	
	// If the media player was unable to open the source, return nullptr to indicate that the media texture will not be displaying the media source
	return nullptr;
}

UMediaTexture* UMediaProfilePlaybackManager::OpenSource(UMediaSource* InMediaSource, void* InConsumer)
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return nullptr;
	}
	
	int32 MediaSourceIndex = MediaProfile->FindMediaSourceIndex(InMediaSource);
	return OpenSourceFromIndex(MediaSourceIndex, InConsumer);
}

bool UMediaProfilePlaybackManager::IsSourceOpenFromIndex(int32 InMediaSourceIndex, void* InConsumer) const
{
	if (!MediaSourceTextures.IsValidIndex(InMediaSourceIndex))
	{
		return false;
	}
	
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return false;
	}
	
	if (InMediaSourceIndex < 0 || InMediaSourceIndex >= MediaProfile->NumMediaSources())
	{
		return false;
	}

	if (!MediaSourcePlayers.Contains(InMediaSourceIndex))
	{
		return false;
	}

	const TObjectPtr<UMediaPlayer>& MediaPlayer = MediaSourcePlayers[InMediaSourceIndex];
	if (!MediaPlayer->GetPlayerFacade()->GetPlayer().IsValid())
	{
		return false;
	}

	const bool bIsOpen = !MediaPlayer->IsClosed();

	if (InConsumer != nullptr)
	{
		const bool bHasConsumer = SourceConsumers.Contains(MediaPlayer) && SourceConsumers[MediaPlayer].Contains(InConsumer);
		return bIsOpen && bHasConsumer;
	}
	
	return bIsOpen;
}

bool UMediaProfilePlaybackManager::IsSourceOpen(UMediaSource* InMediaSource, void* InConsumer) const
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return false;
	}
	
	int32 MediaSourceIndex = MediaProfile->FindMediaSourceIndex(InMediaSource);
	return IsSourceOpenFromIndex(MediaSourceIndex, InConsumer);
}

void UMediaProfilePlaybackManager::CloseSourceFromIndex(int32 InMediaSourceIndex, const FCloseSourceArgs& InArgs)
{
	if (!MediaSourcePlayers.Contains(InMediaSourceIndex))
	{
		return;
	}

	UMediaPlayer* MediaPlayer = MediaSourcePlayers[InMediaSourceIndex];
	UnregisterMediaSourceConsumer(MediaPlayer, InArgs.Consumer);
	
	const bool bStillHasConsumers = GetMediaSourceConsumerCount(MediaPlayer) > 0;
	if ((!bStillHasConsumers && !InArgs.bSoftClose) || InArgs.bForceClose)
	{
		MediaPlayer->Close();
		if (UMediaTexture* MediaTexture = GetSourceMediaTextureFromIndex(InMediaSourceIndex))
		{
			MediaTexture->SetMediaPlayer(nullptr);
		}
	
		if (InArgs.bDestroyMediaPlayer)
		{
			SourceConsumers.Remove(MediaSourcePlayers[InMediaSourceIndex]);
			MediaSourcePlayers.Remove(InMediaSourceIndex);
		}
	}
}

void UMediaProfilePlaybackManager::CloseSource(UMediaSource* InMediaSource, const FCloseSourceArgs& InArgs)
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return;
	}
	
	int32 MediaSourceIndex = MediaProfile->FindMediaSourceIndex(InMediaSource);
	CloseSourceFromIndex(MediaSourceIndex, InArgs);
}

void UMediaProfilePlaybackManager::CloseSourcesForConsumer(const FCloseSourceArgs& InArgs)
{
	TArray<TObjectPtr<UMediaPlayer>> MediaPlayersUsedByConsumer;
	for (const TPair<TObjectPtr<UMediaPlayer>, TSet<void*>>& Pair : SourceConsumers)
	{
		if (Pair.Value.Contains(InArgs.Consumer))
		{
			MediaPlayersUsedByConsumer.Add(Pair.Key);
		}
	}

	for (const TObjectPtr<UMediaPlayer>& MediaPlayer : MediaPlayersUsedByConsumer)
	{
		if (const int32* MediaSourceIndex = MediaSourcePlayers.FindKey(MediaPlayer))
		{
			CloseSourceFromIndex(*MediaSourceIndex, InArgs);
		}
	}
}

TSharedPtr<FViewportClient> UMediaProfilePlaybackManager::GetOrCreateManagedViewportFromIndex(int32 InMediaOutputIndex, EViewModeIndex InViewMode, void* Consumer)
{
	using namespace MediaProfilePlaybackManager;

	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return nullptr;
	}

	UMediaOutput* MediaOutput = MediaProfile->GetMediaOutput(InMediaOutputIndex);
	if (!IsValid(MediaOutput))
	{
		return nullptr;
	}
	
	if (!IsManagedViewportCaptureAllowed())
	{
		return nullptr;
	}

	int32 ExistingViewportIndex = ManagedOutputViewports.IndexOfByPredicate([MediaOutput](const TSharedPtr<FManagedViewport>& InManagedViewport)
	{
		return InManagedViewport->MediaOutput == MediaOutput;
	});

	if (ExistingViewportIndex != INDEX_NONE)
	{
		if (Consumer != nullptr)
		{
			ManagedOutputViewports[ExistingViewportIndex]->Consumers.Add(Consumer);
		}
		
		return ManagedOutputViewports[ExistingViewportIndex]->ViewportClient->AsViewportClient();
	}

#if WITH_EDITOR
	if (ManagedOutputViewports.IsEmpty())
	{
		FEditorDelegates::PostPIEStarted.AddUObject(this, &UMediaProfilePlaybackManager::OnPostPIEStarted);
		FEditorDelegates::PrePIEEnded.AddUObject(this, &UMediaProfilePlaybackManager::OnPrePIEEnded);
	}
#endif
	
	TSharedPtr<FManagedViewport> NewManagedViewport = CreateManagedViewport(MediaOutput, InViewMode);
	
	if (Consumer != nullptr)
	{
		NewManagedViewport->Consumers.Add(Consumer);
	}

	ManagedOutputViewports.Add(NewManagedViewport);

	return NewManagedViewport->ViewportClient->AsViewportClient();
}

TSharedPtr<FViewportClient> UMediaProfilePlaybackManager::GetOrCreateManagedViewport(UMediaOutput* InMediaOutput, EViewModeIndex InViewMode, void* Consumer)
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return nullptr;
	}
	
	int32 MediaOutputIndex = MediaProfile->FindMediaOutputIndex(InMediaOutput);
	return GetOrCreateManagedViewportFromIndex(MediaOutputIndex, InViewMode, Consumer);
}

void UMediaProfilePlaybackManager::ReleaseManagedViewportFromIndex(int32 InMediaOutputIndex, void* Consumer)
{
	using namespace MediaProfilePlaybackManager;

	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return;
	}

	UMediaOutput* MediaOutput = MediaProfile->GetMediaOutput(InMediaOutputIndex);
	if (!IsValid(MediaOutput))
	{
		return;
	}
	
	int32 ExistingViewportIndex = ManagedOutputViewports.IndexOfByPredicate([MediaOutput](const TSharedPtr<FManagedViewport>& InManagedViewport)
	{
		return InManagedViewport->MediaOutput == MediaOutput;
	});

	if (ExistingViewportIndex != INDEX_NONE)
	{
		ManagedOutputViewports[ExistingViewportIndex]->Consumers.Remove(Consumer);
		if (ManagedOutputViewports[ExistingViewportIndex]->Consumers.IsEmpty() && !IsOutputCapturingFromIndex(InMediaOutputIndex))
		{
			ManagedOutputViewports.RemoveAt(ExistingViewportIndex);

#if WITH_EDITOR
			if (ManagedOutputViewports.IsEmpty())
			{
				FEditorDelegates::PostPIEStarted.RemoveAll(this);
				FEditorDelegates::PrePIEEnded.RemoveAll(this);
			}
#endif
		}
	}
}

void UMediaProfilePlaybackManager::ReleaseManagedViewport(UMediaOutput* InMediaOutput, void* Consumer)
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return;
	}
	
	int32 MediaOutputIndex = MediaProfile->FindMediaOutputIndex(InMediaOutput);
	return ReleaseManagedViewportFromIndex(MediaOutputIndex, Consumer);
}

UMediaCapture* UMediaProfilePlaybackManager::OpenManagedViewportOutputFromIndex(int32 InMediaOutputIndex, const FMediaCaptureOptions& InCaptureOptions, bool bAutoRestartCapture)
{
	using namespace MediaProfilePlaybackManager;

	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return nullptr;
	}

	UMediaOutput* MediaOutput = MediaProfile->GetMediaOutput(InMediaOutputIndex);
	if (!IsValid(MediaOutput))
	{
		return nullptr;
	}

	if (!IsManagedViewportCaptureAllowed())
	{
		return nullptr;
	}
	
	int32 ManagedViewportIndex = ManagedOutputViewports.IndexOfByPredicate([MediaOutput](const TSharedPtr<FManagedViewport>& InManagedViewport)
	{
		return InManagedViewport->MediaOutput == MediaOutput;
	});

	if (ManagedViewportIndex == INDEX_NONE)
	{
		checkf(false, TEXT("A managed viewport must have been registered prior to attempting the capture"));
		return nullptr;
	}
	
	UMediaCapture* MediaCapture = FindOrCreateMediaCaptureForIndex(InMediaOutputIndex);
	if (!IsValid(MediaCapture))
	{
		return nullptr;
	}

	// Check to see that the media capture is not already capturing
	EMediaCaptureState CaptureState = MediaCapture->GetState();
	if (CaptureState == EMediaCaptureState::Capturing || CaptureState == EMediaCaptureState::Preparing)
	{
		return MediaCapture;
	};
	
	if (MediaCapture->CaptureSceneViewport(ManagedOutputViewports[ManagedViewportIndex]->Viewport, InCaptureOptions))
	{
#if WITH_EDITOR
		if (bAutoRestartCapture)
		{
			MediaOutput->OnOutputModified().AddUObject(this, &UMediaProfilePlaybackManager::RestartManagedViewportOutput, InCaptureOptions, bAutoRestartCapture);
		}
#endif
		
		return MediaCapture;
	}

	// If the capture failed to start, mark the capture as errored, clean up its capture resources, and return it
	CapturesWithError.Add(MediaCapture);
	CleanUpCaptureResources(MediaOutput);
	
	return MediaCapture;
}

UMediaCapture* UMediaProfilePlaybackManager::OpenManagedViewportOutput(UMediaOutput* InMediaOutput, const FMediaCaptureOptions& InCaptureOptions, bool bAutoRestartCapture)
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return nullptr;
	}
	
	int32 MediaOutputIndex = MediaProfile->FindMediaOutputIndex(InMediaOutput);
	return OpenManagedViewportOutputFromIndex(MediaOutputIndex, InCaptureOptions, bAutoRestartCapture);
}

TSharedPtr<FSceneViewport> UMediaProfilePlaybackManager::GetActiveViewportFromIndex(int32 InMediaOutputIndex) const
{
	using namespace MediaProfilePlaybackManager;

	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return nullptr;
	}

	UMediaOutput* MediaOutput = MediaProfile->GetMediaOutput(InMediaOutputIndex);
	if (!IsValid(MediaOutput))
	{
		return nullptr;
	}
	
	if (!IsActiveViewportCaptureAllowed())
	{
		return nullptr;
	}

	// First, look for any stored viewports that are actively being captured for the media output, and return that if one is found
	int32 ActiveViewportIndex = ActiveOutputViewports.IndexOfByPredicate([MediaOutput](const TSharedPtr<FActiveViewport>& InViewport)
	{
		return InViewport->MediaOutput == MediaOutput;
	});

	if (ActiveViewportIndex != INDEX_NONE)
	{
		return ActiveOutputViewports[ActiveViewportIndex]->GetSceneViewport();
	}

	// Otherwise, query the engine and find an active viewport to return
	if (TSharedPtr<FSceneViewport> ActiveViewport = FindActiveViewport())
	{
		return ActiveViewport;
	}
			
#if WITH_EDITOR
	if (TSharedPtr<IAssetViewport> ActiveViewportInterface = FindActiveViewportInterface())
	{
		return ActiveViewportInterface->GetSharedActiveViewport();
	}
#endif

	return nullptr;
}

TSharedPtr<FSceneViewport> UMediaProfilePlaybackManager::GetActiveViewport(UMediaOutput* InMediaOutput) const
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return nullptr;
	}
	
	int32 MediaOutputIndex = MediaProfile->FindMediaOutputIndex(InMediaOutput);
	return GetActiveViewportFromIndex(MediaOutputIndex);
}

UMediaCapture* UMediaProfilePlaybackManager::OpenActiveViewportOutputFromIndex(int32 InMediaOutputIndex, const FMediaCaptureOptions& InCaptureOptions, bool bAutoRestartCapture)
{
	using namespace MediaProfilePlaybackManager;
	
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return nullptr;
	}

	UMediaOutput* MediaOutput = MediaProfile->GetMediaOutput(InMediaOutputIndex);
	if (!IsValid(MediaOutput))
	{
		return nullptr;
	}
	
	if (!IsActiveViewportCaptureAllowed())
	{
		return nullptr;
	}

	UMediaCapture* MediaCapture = FindOrCreateMediaCaptureForIndex(InMediaOutputIndex);
	if (!IsValid(MediaCapture))
	{
		return nullptr;
	}
	
	// Check to see that the media capture is not already capturing
	EMediaCaptureState CaptureState = MediaCapture->GetState();
	if (CaptureState != EMediaCaptureState::Stopped && CaptureState != EMediaCaptureState::StopRequested && CaptureState != EMediaCaptureState::Error)
	{
		return MediaCapture;
	};

	int32 ActiveViewportIndex = ActiveOutputViewports.IndexOfByPredicate([MediaOutput](const TSharedPtr<FActiveViewport>& InViewport)
	{
		return InViewport->MediaOutput == MediaOutput;
	});

	if (ActiveViewportIndex == INDEX_NONE)
	{
#if WITH_EDITOR
		if (ActiveOutputViewports.IsEmpty())
		{
			if (GEditor)
			{
				GEditor->OnLevelViewportClientListChanged().AddUObject(this, &UMediaProfilePlaybackManager::OnLevelViewportClientListChanged);
			}
		}
#endif
		
		ActiveViewportIndex = ActiveOutputViewports.Add(MakeShared<FActiveViewport>(MediaOutput, InCaptureOptions));
	}

	TSharedPtr<FSceneViewport> ActiveViewport = ActiveOutputViewports[ActiveViewportIndex]->GetSceneViewport();
	if (MediaCapture->CaptureSceneViewport(ActiveViewport, InCaptureOptions))
	{
#if WITH_EDITOR
		if (bAutoRestartCapture)
		{
			MediaOutput->OnOutputModified().AddUObject(this, &UMediaProfilePlaybackManager::RestartActiveViewportOutput, InCaptureOptions, bAutoRestartCapture);
		}
#endif
		
		return MediaCapture;
	}

	// If the capture failed to start, mark the capture as errored, clean up its capture resources, and return it
	CapturesWithError.Add(MediaCapture);
	CleanUpCaptureResources(MediaOutput);
	
	return MediaCapture;
}

UMediaCapture* UMediaProfilePlaybackManager::OpenActiveViewportOutput(UMediaOutput* InMediaOutput, const FMediaCaptureOptions& InCaptureOptions, bool bAutoRestartCapture)
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return nullptr;
	}
	
	int32 MediaOutputIndex = MediaProfile->FindMediaOutputIndex(InMediaOutput);
	return OpenActiveViewportOutputFromIndex(MediaOutputIndex, InCaptureOptions, bAutoRestartCapture);
}

UMediaCapture* UMediaProfilePlaybackManager::OpenRenderTargetOutputFromIndex(int32 InMediaOutputIndex, UTextureRenderTarget2D* InRenderTarget, const FMediaCaptureOptions& InCaptureOptions, bool bAutoRestartCapture)
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return nullptr;
	}

	UMediaOutput* MediaOutput = MediaProfile->GetMediaOutput(InMediaOutputIndex);
	if (!IsValid(MediaOutput))
	{
		return nullptr;
	}
	
	UMediaCapture* MediaCapture = FindOrCreateMediaCaptureForIndex(InMediaOutputIndex);
	if (!IsValid(MediaCapture))
	{
		return nullptr;
	}
	
	// Check to see that the media capture is not already capturing
	EMediaCaptureState CaptureState = MediaCapture->GetState();
	if (CaptureState != EMediaCaptureState::Stopped && CaptureState != EMediaCaptureState::StopRequested && CaptureState != EMediaCaptureState::Error)
	{
		return MediaCapture;
	};

	if (MediaCapture->CaptureTextureRenderTarget2D(InRenderTarget, InCaptureOptions))
	{
#if WITH_EDITOR
		if (bAutoRestartCapture)
		{
			MediaOutput->OnOutputModified().AddUObject(this, &UMediaProfilePlaybackManager::RestartRenderTargetOutput, InRenderTarget, InCaptureOptions, bAutoRestartCapture);
		}
#endif
		
		return MediaCapture;
	}

	// If the capture failed to start, mark the capture as errored, clean up its capture resources, and return it
	CapturesWithError.Add(MediaCapture);
	CleanUpCaptureResources(MediaOutput);
	
	return MediaCapture;
}

UMediaCapture* UMediaProfilePlaybackManager::OpenRenderTargetOutput(UMediaOutput* InMediaOutput, UTextureRenderTarget2D* InRenderTarget, const FMediaCaptureOptions& InCaptureOptions, bool bAutoRestartCapture)
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return nullptr;
	}
	
	int32 MediaOutputIndex = MediaProfile->FindMediaOutputIndex(InMediaOutput);
	return OpenRenderTargetOutputFromIndex(MediaOutputIndex, InRenderTarget, InCaptureOptions, bAutoRestartCapture);
}

UMediaCapture* UMediaProfilePlaybackManager::OpenMediaTextureOutputFromIndex(int32 InMediaOutputIndex, UMediaTexture* InMediaTexture, const FMediaCaptureOptions& InCaptureOptions, const FMediaCaptureTransform& InTransform, bool bAutoRestartCapture)
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return nullptr;
	}

	UMediaOutput* MediaOutput = MediaProfile->GetMediaOutput(InMediaOutputIndex);
	if (!MediaOutput)
	{
		return nullptr;
	}
	
	UMediaCapture* MediaCapture = FindOrCreateMediaCaptureForIndex(InMediaOutputIndex);
	if (!MediaCapture)
	{
		return nullptr;
	}
	
	// Check to see that the media capture is not already capturing
	EMediaCaptureState CaptureState = MediaCapture->GetState();
	if (CaptureState != EMediaCaptureState::Stopped && CaptureState != EMediaCaptureState::StopRequested && CaptureState != EMediaCaptureState::Error)
	{
		return MediaCapture;
	};

	if (MediaCapture->CaptureMediaTexture(InMediaTexture, InCaptureOptions, InTransform.GetRotationValue(true), InTransform.GetScaleValue()))
	{
		int32 MediaTextureCaptureIndex = MediaTextureCaptures.IndexOfByPredicate([MediaOutput](const TSharedPtr<MediaProfilePlaybackManager::FMediaTextureCapture>& InOutputMediaTexture)
		{
			return InOutputMediaTexture->MediaOutput == MediaOutput;
		});

		if (MediaTextureCaptureIndex == INDEX_NONE)
		{
			MediaTextureCaptureIndex = MediaTextureCaptures.Add(MakeShared<MediaProfilePlaybackManager::FMediaTextureCapture>(MediaOutput));
		}

		MediaTextureCaptures[MediaTextureCaptureIndex]->UpdateMediaTexture(InMediaTexture, InCaptureOptions, InTransform, bAutoRestartCapture);

#if WITH_EDITOR
		if (bAutoRestartCapture)
		{
			MediaOutput->OnOutputModified().AddUObject(this, &UMediaProfilePlaybackManager::RestartMediaTextureOutput, InMediaTexture, InCaptureOptions, InTransform, bAutoRestartCapture);
		}
#endif
		
		return MediaCapture;
	}

	// If the capture failed to start, mark the capture as errored, clean up its capture resources, and return it
	CapturesWithError.Add(MediaCapture);
	CleanUpCaptureResources(MediaOutput);
	
	return MediaCapture;
}

UMediaCapture* UMediaProfilePlaybackManager::OpenMediaTextureOutput(UMediaOutput* InMediaOutput, UMediaTexture* InMediaTexture, const FMediaCaptureOptions& InCaptureOptions, const FMediaCaptureTransform& InTransform, bool bAutoRestartCapture)
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return nullptr;
	}
	
	int32 MediaOutputIndex = MediaProfile->FindMediaOutputIndex(InMediaOutput);
	return OpenMediaTextureOutputFromIndex(MediaOutputIndex, InMediaTexture, InCaptureOptions, InTransform, bAutoRestartCapture);
}

void UMediaProfilePlaybackManager::RestartActiveViewportOutput(UMediaOutput* InMediaOutput, FMediaCaptureOptions InCaptureOptions, bool bAutoRestartCapture)
{
	TWeakObjectPtr<UMediaOutput> WeakMediaOutput = InMediaOutput;
	auto Callback = [WeakMediaOutput, InCaptureOptions, bAutoRestartCapture](UMediaProfilePlaybackManager* Manager)
	{
		TStrongObjectPtr<UMediaOutput> MediaOutput = WeakMediaOutput.Pin();

		if (Manager && MediaOutput)
		{
			Manager->OpenActiveViewportOutput(MediaOutput.Get(), InCaptureOptions, bAutoRestartCapture);
		}
	};
	QueueRestartCapture(InMediaOutput, Callback);
}

void UMediaProfilePlaybackManager::RestartManagedViewportOutput(UMediaOutput* InMediaOutput, FMediaCaptureOptions InCaptureOptions, bool bAutoRestartCapture)
{
	TWeakObjectPtr<UMediaOutput> WeakMediaOutput = InMediaOutput;
	auto Callback = [WeakMediaOutput, InCaptureOptions, bAutoRestartCapture](UMediaProfilePlaybackManager* Manager)
	{
		TStrongObjectPtr<UMediaOutput> MediaOutput = WeakMediaOutput.Pin();

		if (Manager && MediaOutput)
		{
			Manager->OpenManagedViewportOutput(MediaOutput.Get(), InCaptureOptions, bAutoRestartCapture);
		}
	};
	QueueRestartCapture(InMediaOutput, Callback);
}

void UMediaProfilePlaybackManager::RestartRenderTargetOutput(UMediaOutput* InMediaOutput, UTextureRenderTarget2D* InRenderTarget, FMediaCaptureOptions InCaptureOptions, bool bAutoRestartCapture)
{
	TWeakObjectPtr<UTextureRenderTarget2D> WeakRenderTarget = InRenderTarget;
	TWeakObjectPtr<UMediaOutput> WeakMediaOutput = InMediaOutput;
	auto Callback = [WeakRenderTarget, WeakMediaOutput, InCaptureOptions, bAutoRestartCapture](UMediaProfilePlaybackManager* Manager)
	{
		TStrongObjectPtr<UTextureRenderTarget2D> RenderTarget = WeakRenderTarget.Pin();
		TStrongObjectPtr<UMediaOutput> MediaOutput = WeakMediaOutput.Pin();

		if (Manager && RenderTarget && MediaOutput)
		{
			Manager->OpenRenderTargetOutput(MediaOutput.Get(), RenderTarget.Get(), InCaptureOptions, bAutoRestartCapture);
		}
	};
	QueueRestartCapture(InMediaOutput, Callback);
}

void UMediaProfilePlaybackManager::RestartMediaTextureOutput(UMediaOutput* InMediaOutput, UMediaTexture* InMediaTexture, FMediaCaptureOptions InCaptureOptions, FMediaCaptureTransform InTransform, bool bAutoRestartCapture)
{
	TWeakObjectPtr<UMediaTexture> WeakMediaTexture = InMediaTexture;
	TWeakObjectPtr<UMediaOutput> WeakMediaOutput = InMediaOutput;
	auto Callback = [WeakMediaTexture, WeakMediaOutput, InCaptureOptions, InTransform, bAutoRestartCapture](UMediaProfilePlaybackManager* Manager)
	{
		TStrongObjectPtr<UMediaTexture> MediaTexture = WeakMediaTexture.Pin();
		TStrongObjectPtr<UMediaOutput> MediaOutput = WeakMediaOutput.Pin();

		if (Manager && MediaTexture && MediaOutput)
		{
			Manager->OpenMediaTextureOutput(MediaOutput.Get(), MediaTexture.Get(), InCaptureOptions, InTransform, bAutoRestartCapture);
		}
	};
	QueueRestartCapture(InMediaOutput, Callback);
}

bool UMediaProfilePlaybackManager::IsOutputCapturingFromIndex(int32 InMediaOutputIndex) const
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return false;
	}

	UMediaOutput* MediaOutput = MediaProfile->GetMediaOutput(InMediaOutputIndex);
	if (!IsValid(MediaOutput))
	{
		return false;
	}
	
	if (!MediaOutputCaptures.Contains(InMediaOutputIndex))
	{
		return false;
	}
	
	UMediaCapture* MediaCapture = MediaOutputCaptures[InMediaOutputIndex];
	if (MediaCapture->GetState() == EMediaCaptureState::Stopped || MediaCapture->GetState() == EMediaCaptureState::Error)
	{
		return false;
	}
	
	return MediaCapture->GetMediaOutputName() == MediaOutput->GetName();
}

bool UMediaProfilePlaybackManager::IsOutputCapturing(UMediaOutput* InMediaOutput) const
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return false;
	}

	int32 MediaOutputIndex = MediaProfile->FindMediaOutputIndex(InMediaOutput);
	return IsOutputCapturingFromIndex(MediaOutputIndex);
}

TOptional<EMediaCaptureState> UMediaProfilePlaybackManager::GetOutputCaptureStateFromIndex(int32 InMediaOutputIndex, bool& bOutHasError) const
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return TOptional<EMediaCaptureState>();
	}

	if (!MediaOutputCaptures.Contains(InMediaOutputIndex))
	{
		return TOptional<EMediaCaptureState>();
	}

	UMediaCapture* MediaCapture = MediaOutputCaptures[InMediaOutputIndex];

	bOutHasError = CapturesWithError.Contains(MediaCapture);
	return MediaCapture->GetState();
}

TOptional<EMediaCaptureState> UMediaProfilePlaybackManager::GetOutputCaptureState(UMediaOutput* InMediaOutput, bool& bOutHasError) const
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return TOptional<EMediaCaptureState>();
	}

	int32 MediaOutputIndex = MediaProfile->FindMediaOutputIndex(InMediaOutput);
	return GetOutputCaptureStateFromIndex(MediaOutputIndex, bOutHasError);
}

void UMediaProfilePlaybackManager::CloseOutputFromIndex(int32 InMediaOutputIndex, const FCloseOutputArgs& InArgs)
{
	using namespace MediaProfilePlaybackManager;

	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return;
	}

	UMediaOutput* MediaOutput = MediaProfile->GetMediaOutput(InMediaOutputIndex);
	if (!IsValid(MediaOutput))
	{
		return;
	}

	if (!MediaOutputCaptures.Contains(InMediaOutputIndex))
	{
		return;
	}

	UMediaCapture* MediaCapture = MediaOutputCaptures[InMediaOutputIndex];

	constexpr bool bAllowPendingFrameToBeProcessed = false;
	MediaCapture->StopCapture_Callback(bAllowPendingFrameToBeProcessed, FSimpleDelegate::CreateUObject(this, &UMediaProfilePlaybackManager::PostStopCapture, InMediaOutputIndex, InArgs));
}

void UMediaProfilePlaybackManager::PostStopCapture(int32 InMediaOutputIndex, FCloseOutputArgs InArgs)
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return;
	}

	UMediaOutput* MediaOutput = MediaProfile->GetMediaOutput(InMediaOutputIndex);
	if (!IsValid(MediaOutput))
	{
		return;
	}

	if (!MediaOutputCaptures.Contains(InMediaOutputIndex))
	{
		return;
	}

	UMediaCapture* MediaCapture = MediaOutputCaptures[InMediaOutputIndex];

	if (InArgs.bDestroyCaptureObjects)
	{
		CleanUpCaptureResources(MediaOutput);

		MediaCapture->OnStateChangedNative.RemoveAll(this);
		MediaOutputCaptures.Remove(InMediaOutputIndex);
	}

#if WITH_EDITOR
	MediaOutput->OnOutputModified().RemoveAll(this);
#endif

	InArgs.Callback.ExecuteIfBound();
}

void UMediaProfilePlaybackManager::CloseOutput(UMediaOutput* InMediaOutput, const FCloseOutputArgs& InArgs)
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return;
	}
	
	int32 MediaOutputIndex = MediaProfile->FindMediaOutputIndex(InMediaOutput);
	CloseOutputFromIndex(MediaOutputIndex, InArgs);
}

UMediaPlayer* UMediaProfilePlaybackManager::FindOrCreateMediaPlayerForIndex(int32 InMediaSourceIndex)
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return nullptr;
	}
	
	if (InMediaSourceIndex < 0 || InMediaSourceIndex >= MediaProfile->NumMediaSources())
	{
		return nullptr;
	}

	if (!MediaSourcePlayers.Contains(InMediaSourceIndex))
	{
		UMediaPlayer* NewMediaPlayer = NewObject<UMediaPlayer>(this);

#if WITH_EDITOR
		NewMediaPlayer->AffectedByPIEHandling = false;
#endif
		
		MediaSourcePlayers.Add(InMediaSourceIndex, NewMediaPlayer);
	}

	return MediaSourcePlayers[InMediaSourceIndex];
}

void UMediaProfilePlaybackManager::ChangeMediaSourceIndex(int32 InOriginalMediaSourceIndex, int32 InNewMediaSourceIndex)
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return;
	}

	if (InOriginalMediaSourceIndex == InNewMediaSourceIndex)
	{
		return;
	}
	
	// First, remove the existing media player from the list of media players and shift all slots below it up one
	UMediaPlayer* MediaPlayer = nullptr;
	if (InOriginalMediaSourceIndex != INDEX_NONE)
	{
		MediaPlayer = MediaSourcePlayers.Contains(InOriginalMediaSourceIndex) ? MediaSourcePlayers[InOriginalMediaSourceIndex] : nullptr;
		int32 LastIndex = INDEX_NONE;
		for (int32 Index = InOriginalMediaSourceIndex + 1; Index < MediaProfile->NumMediaSources(); ++Index)
		{
			if (MediaSourcePlayers.Contains(Index))
			{
				MediaSourcePlayers.Add(Index - 1, MediaSourcePlayers[Index]);
				LastIndex = Index;
			}
		}

		if (LastIndex != INDEX_NONE)
		{
			MediaSourcePlayers.Remove(LastIndex);
		}
	}

	// Second, insert the media player back into the list, shifting all slots at or below that index down one
	if (InNewMediaSourceIndex != INDEX_NONE)
	{
		for (int32 Index = MediaProfile->NumMediaSources() - 1; Index >= InNewMediaSourceIndex; --Index)
		{
			if (MediaSourcePlayers.Contains(Index))
			{
				MediaSourcePlayers.Add(Index + 1, MediaSourcePlayers[Index]);
			}
		}

		if (MediaPlayer)
		{
			MediaSourcePlayers.Add(InNewMediaSourceIndex, MediaPlayer);
		}
	}
	else
	{
		SourceConsumers.Remove(MediaPlayer);
	}

	// Finally, refresh all media textures to have the correct media player attached
	for (int32 Index = 0; Index < MediaSourceTextures.Num(); ++Index)
	{
		if (UMediaTexture* MediaTexture = GetSourceMediaTextureFromIndex(Index))
		{
			if (MediaSourcePlayers.Contains(Index))
			{
				MediaTexture->SetMediaPlayer(MediaSourcePlayers[Index]);
			}
			else
			{
				MediaTexture->SetMediaPlayer(nullptr);
			}
		}
	}
}

UMediaProfile* UMediaProfilePlaybackManager::GetOwningMediaProfile() const
{
	return GetTypedOuter<UMediaProfile>();
}

void UMediaProfilePlaybackManager::RegisterMediaSourceConsumer(UMediaPlayer* InMediaPlayer, void* InConsumer)
{
	if (InConsumer)
	{
		if (SourceConsumers.Contains(InMediaPlayer))
		{
			SourceConsumers[InMediaPlayer].Add(InConsumer);
		}
		else
		{
			SourceConsumers.Add(InMediaPlayer, TSet<void*> { InConsumer });
		}
	}
}

void UMediaProfilePlaybackManager::UnregisterMediaSourceConsumer(UMediaPlayer* InMediaPlayer, void* InConsumer)
{
	if (InConsumer)
	{
		if (SourceConsumers.Contains(InMediaPlayer))
		{
			SourceConsumers[InMediaPlayer].Remove(InConsumer);
		}
	}
}

int32 UMediaProfilePlaybackManager::GetMediaSourceConsumerCount(UMediaPlayer* InMediaPlayer) const
{
	if (SourceConsumers.Contains(InMediaPlayer))
	{
		return SourceConsumers[InMediaPlayer].Num();
	}

	return 0;
}

UMediaCapture* UMediaProfilePlaybackManager::FindOrCreateMediaCaptureForIndex(int32 InMediaOutputIndex)
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return nullptr;
	}
	
	UMediaOutput* MediaOutput = MediaProfile->GetMediaOutput(InMediaOutputIndex);
	if (!IsValid(MediaOutput))
	{
		return nullptr;
	}

	if (!MediaOutputCaptures.Contains(InMediaOutputIndex))
	{
		UMediaCapture* NewMediaCapture = MediaOutput->CreateMediaCapture();
		if (!IsValid(NewMediaCapture))
		{
			return nullptr;
		}

		NewMediaCapture->OnStateChangedNative.AddUObject(this, &UMediaProfilePlaybackManager::OnCaptureStateChanged, NewMediaCapture);
		
		MediaOutputCaptures.Add(InMediaOutputIndex, NewMediaCapture);
	}

	return MediaOutputCaptures[InMediaOutputIndex];
}

void UMediaProfilePlaybackManager::ChangeMediaOutputIndex(int32 InOriginalMediaOutputIndex, int32 InNewMediaOutputIndex)
{
	UMediaProfile* MediaProfile = GetOwningMediaProfile();
	if (!MediaProfile)
	{
		return;
	}

	if (InOriginalMediaOutputIndex == InNewMediaOutputIndex)
	{
		return;
	}

	// First, remove the existing media player from the list of media players and shift all slots below it up one
	UMediaCapture* MediaCapture = nullptr;
	if (InOriginalMediaOutputIndex != INDEX_NONE)
	{
		MediaCapture = MediaOutputCaptures.Contains(InOriginalMediaOutputIndex) ? MediaOutputCaptures[InOriginalMediaOutputIndex] : nullptr;
		int32 LastIndex = INDEX_NONE;
		for (int32 Index = InOriginalMediaOutputIndex + 1; Index < MediaProfile->NumMediaOutputs(); ++Index)
		{
			if (MediaOutputCaptures.Contains(Index))
			{
				MediaOutputCaptures.Add(Index - 1, MediaOutputCaptures[Index]);
				LastIndex = Index;
			}
		}

		if (LastIndex != INDEX_NONE)
		{
			MediaOutputCaptures.Remove(LastIndex);
		}
	}

	// Second, insert the media player back into the list, shifting all slots at or below that index down one
	if (InNewMediaOutputIndex != INDEX_NONE)
	{
		for (int32 Index = MediaProfile->NumMediaOutputs() - 1; Index >= InNewMediaOutputIndex; --Index)
		{
			if (MediaOutputCaptures.Contains(Index))
			{
				MediaOutputCaptures.Add(Index + 1, MediaOutputCaptures[Index]);
			}
		}

		if (MediaCapture)
		{
			MediaOutputCaptures.Add(InNewMediaOutputIndex, MediaCapture);
		}
	}
}

void UMediaProfilePlaybackManager::OnCaptureStateChanged(UMediaCapture* InMediaCapture)
{
	if (InMediaCapture->GetState() == EMediaCaptureState::Error)
	{
		CapturesWithError.Add(InMediaCapture);
	}
	else if (InMediaCapture->GetState() == EMediaCaptureState::Preparing)
	{
		CapturesWithError.Remove(InMediaCapture);
	}
	else if (InMediaCapture->GetState() == EMediaCaptureState::Stopped)
	{
		// If the capture has been stopped due to an error, try and clean up any resources the media capture was using
		if (CapturesWithError.Contains(InMediaCapture))
		{
			if (const int32* MediaOutputIndex = MediaOutputCaptures.FindKey(InMediaCapture))
			{
				UMediaProfile* MediaProfile = GetOwningMediaProfile();
				if (!MediaProfile)
				{
					return;
				}
				
				UMediaOutput* MediaOutput = MediaProfile->GetMediaOutput(*MediaOutputIndex);
				if (!IsValid(MediaOutput))
				{
					return;
				}
				
				CleanUpCaptureResources(MediaOutput);
			}
		}
	}
}

void UMediaProfilePlaybackManager::QueueRestartCapture(UMediaOutput* InMediaOutput, FRecaptureFunc InRestartCaptureFunc)
{
	RestartCaptureFuncs.Add(InRestartCaptureFunc);

	constexpr bool bDestroyCaptureObjects = false;

	FCloseOutputArgs Args;
	Args.bDestroyCaptureObjects = false;
	Args.Callback = FSimpleDelegate::CreateUObject(this, &UMediaProfilePlaybackManager::DeferredRestart);

	CloseOutput(InMediaOutput, MoveTemp(Args));
}

void UMediaProfilePlaybackManager::DeferredRestart()
{
	// Not all implementations of MediaCapture implementations have a StopCapture_Future method, 
	// so we defer to the next tick to make sure those MediaCaptures also have a chance to shutdown before he output is restarted.
	if (!RestartCapturesTimerHandle.IsValid())
	{
#if WITH_EDITOR
		if (GEditor)
		{
			RestartCapturesTimerHandle = GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateUObject(this, &UMediaProfilePlaybackManager::RestartCaptures));
		}
		else
#endif
		if (UWorld* World = GetWorld())
		{
			RestartCapturesTimerHandle = World->GetTimerManager().SetTimerForNextTick(FTimerDelegate::CreateUObject(this, &UMediaProfilePlaybackManager::RestartCaptures));
		}
	}
}

void UMediaProfilePlaybackManager::RestartCaptures()
{
	for (FRecaptureFunc& RestartCaptureFunc : RestartCaptureFuncs)
	{
		RestartCaptureFunc(this);
	}

	RestartCaptureFuncs.Empty();
	RestartCapturesTimerHandle.Invalidate();
}
 
void UMediaProfilePlaybackManager::Cleanup()
{
	TArray<int32> OpenSources;
	MediaSourcePlayers.GetKeys(OpenSources);
	for (int32 MediaSourceIndex : OpenSources)
	{
		FCloseSourceArgs Args;
		Args.bForceClose = true;
		Args.bDestroyMediaPlayer = true;
		
		CloseSourceFromIndex(MediaSourceIndex, Args);
	}

	TArray<int32> OpenOutputs;
	MediaOutputCaptures.GetKeys(OpenOutputs);
	for (int32 MediaOutputIndex : OpenOutputs)
	{
		CloseOutputFromIndex(MediaOutputIndex);
	}

	ManagedOutputViewports.Empty();
	ActiveOutputViewports.Empty();
	MediaTextureCaptures.Empty();

#if WITH_EDITOR
	FEditorDelegates::PostPIEStarted.RemoveAll(this);
	FEditorDelegates::PrePIEEnded.RemoveAll(this);
	
	if (GEditor)
	{
		GEditor->OnLevelViewportClientListChanged().RemoveAll(this);
	}
#endif
}

void UMediaProfilePlaybackManager::CleanUpCaptureResources(UMediaOutput* InMediaOutput)
{
	ManagedOutputViewports.RemoveAll([InMediaOutput](const TSharedPtr<MediaProfilePlaybackManager::FManagedViewport>& InManagedViewport)
		{
			return InManagedViewport->MediaOutput == InMediaOutput && InManagedViewport->Consumers.IsEmpty();
		});

	ActiveOutputViewports.RemoveAll([InMediaOutput](const TSharedPtr<MediaProfilePlaybackManager::FActiveViewport>& InViewport)
	{
		return InViewport->MediaOutput == InMediaOutput;
	});

	MediaTextureCaptures.RemoveAll([InMediaOutput](const TSharedPtr<MediaProfilePlaybackManager::FMediaTextureCapture>& InMediaTextureCapture)
	{
		return !InMediaTextureCapture.IsValid() || InMediaTextureCapture->MediaOutput.Get() == InMediaOutput;
	});

#if WITH_EDITOR
	if (ManagedOutputViewports.IsEmpty())
	{
		FEditorDelegates::PostPIEStarted.RemoveAll(this);
		FEditorDelegates::PrePIEEnded.RemoveAll(this);
	}

	if (ActiveOutputViewports.IsEmpty())
	{
		if (GEditor)
		{
			GEditor->OnLevelViewportClientListChanged().RemoveAll(this);
		}
	}
#endif
}

#if WITH_EDITOR
void UMediaProfilePlaybackManager::OnPostPIEStarted(bool bIsSimulating)
{
	using namespace MediaProfilePlaybackManager;
	
	for (const TSharedPtr<FManagedViewport>& ManagedViewport : ManagedOutputViewports)
	{
		if (ManagedViewport->ViewportClient.IsValid())
		{
			ManagedViewport->ViewportClient->SetPIE(true);
		}
	}
}

void UMediaProfilePlaybackManager::OnPrePIEEnded(bool bIsSimulating)
{
	using namespace MediaProfilePlaybackManager;
	
	for (const TSharedPtr<FManagedViewport>& ManagedViewport : ManagedOutputViewports)
	{
		if (ManagedViewport->ViewportClient.IsValid())
		{
			ManagedViewport->ViewportClient->SetPIE(false);
		}
	}
}

void UMediaProfilePlaybackManager::OnLevelViewportClientListChanged()
{
	using namespace MediaProfilePlaybackManager;

	if (!GEditor)
	{
		return;
	}
	
	for (const TSharedPtr<FActiveViewport>& ActiveViewport : ActiveOutputViewports)
	{
		bool bStillActive = false;
		if (TSharedPtr<FSceneViewport> Viewport = ActiveViewport->GetSceneViewport())
		{
			for (FLevelEditorViewportClient* ViewportClient : GEditor->GetLevelViewportClients())
			{
				if (ViewportClient->Viewport == Viewport.Get())
				{
					bStillActive = true;
					break;
				}
			}
		}

		if (!bStillActive)
		{
			CloseOutput(ActiveViewport->MediaOutput.Get());
		}
	}
}
#endif
