// Copyright Epic Games, Inc. All Rights Reserved.


#include "Profile/MediaProfileSettings.h"

#include "MediaAssets/ProxyMediaSource.h"
#include "MediaAssets/ProxyMediaOutput.h"
#include "Profile/MediaProfile.h"

namespace MediaProfileSettings
{
	const FString DefaultProxiesPath = TEXT("/MediaFrameworkUtilities/DefaultProxies");
	const FString DefaultMediaProxySourceName = TEXT("DefaultMediaProxySource");
	const FString DefaultMediaProxyOutputName = TEXT("DefaultMediaProxyOutput");
}

TArray<UProxyMediaSource*> UMediaProfileSettings::LoadMediaSourceProxies() const
{
	TArray<UProxyMediaSource*> MediaSourceProxyPtr;
	MediaSourceProxyPtr.Reset(MediaSourceProxy.Num());

	for (const TSoftObjectPtr<UProxyMediaSource>& Proxy : MediaSourceProxy)
	{
		MediaSourceProxyPtr.Add(Proxy.LoadSynchronous());
	}
	return MediaSourceProxyPtr;
}


TArray<UProxyMediaOutput*> UMediaProfileSettings::LoadMediaOutputProxies() const
{
	TArray<UProxyMediaOutput*> MediaOutputProxyPtr;
	MediaOutputProxyPtr.Reset(MediaOutputProxy.Num());

	for (const TSoftObjectPtr<UProxyMediaOutput>& Proxy : MediaOutputProxy)
	{
		MediaOutputProxyPtr.Add(Proxy.LoadSynchronous());
	}
	return MediaOutputProxyPtr;
}


UMediaProfile* UMediaProfileSettings::GetStartupMediaProfile() const
{
	return StartupMediaProfile.LoadSynchronous();
}


#if WITH_EDITOR
void UMediaProfileSettings::SetMediaSourceProxy(const TArray<UProxyMediaSource*>& InProxies)
{
	MediaSourceProxy.Reset();
	for (UProxyMediaSource* Proxy : InProxies)
	{
		MediaSourceProxy.Add(Proxy);
	}
	OnMediaProxiesChanged.Broadcast();
}

void UMediaProfileSettings::FillDefaultMediaSourceProxies(int32 NumSources, bool bSilent)
{
	MediaSourceProxy.Reset(NumSources);
	for (int32 Index = 1; Index <= NumSources; ++Index)
	{
		FString IndexStr = Index > 9 ? FString::FromInt(Index) : TEXT("0") + FString::FromInt(Index);
		FSoftObjectPath ProxyPath(FString::Format(TEXT("{0}/{1}_{2}"), { MediaProfileSettings::DefaultProxiesPath, MediaProfileSettings::DefaultMediaProxySourceName, IndexStr }));
		UProxyMediaSource* DefaultProxySource = Cast<UProxyMediaSource>(ProxyPath.TryLoad());
		check(DefaultProxySource);

		MediaSourceProxy.Add(DefaultProxySource);
	}

	if (!bSilent)
	{
		OnMediaProxiesChanged.Broadcast();
	}
}

void UMediaProfileSettings::SetMediaOutputProxy(const TArray<UProxyMediaOutput*>& InProxies)
{
	MediaOutputProxy.Reset();
	for (UProxyMediaOutput* Proxy : InProxies)
	{
		MediaOutputProxy.Add(Proxy);
	}
	OnMediaProxiesChanged.Broadcast();
}

void UMediaProfileSettings::FillDefaultMediaOutputProxies(int32 NumOutputs, bool bSilent)
{
	MediaOutputProxy.Reset(NumOutputs);
	for (int32 Index = 1; Index <= NumOutputs; ++Index)
	{
		FString IndexStr = Index > 9 ? FString::FromInt(Index) : TEXT("0") + FString::FromInt(Index);
		FSoftObjectPath ProxyPath(FString::Format(TEXT("{0}/{1}_{2}"), { MediaProfileSettings::DefaultProxiesPath, MediaProfileSettings::DefaultMediaProxyOutputName, IndexStr }));
		UProxyMediaOutput* DefaultProxyOutput = Cast<UProxyMediaOutput>(ProxyPath.TryLoad());
		check(DefaultProxyOutput);

		MediaOutputProxy.Add(DefaultProxyOutput);
	}

	if (!bSilent)
	{
		OnMediaProxiesChanged.Broadcast();
	}
}

void UMediaProfileSettings::PostEditChangeProperty(struct FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (GET_MEMBER_NAME_CHECKED(ThisClass, MediaOutputProxy) == InPropertyChangedEvent.GetPropertyName()
		|| GET_MEMBER_NAME_CHECKED(ThisClass, MediaSourceProxy) == InPropertyChangedEvent.GetPropertyName())
	{
		// Recache the proxies ptr
		OnMediaProxiesChanged.Broadcast();
	}

	Super::PostEditChangeProperty(InPropertyChangedEvent);
}
#endif // WITH_EDITOR


UMediaProfileEditorSettings::UMediaProfileEditorSettings()
#if WITH_EDITOR
	: bDisplayInToolbar(true)
	, bDisplayInMainEditor(false)
#endif
{
}


UMediaProfile* UMediaProfileEditorSettings::GetUserMediaProfile() const
{
#if WITH_EDITOR
	return UserMediaProfile.LoadSynchronous();
#else
	return nullptr;
#endif
	
}


void UMediaProfileEditorSettings::SetUserMediaProfile(UMediaProfile* InMediaProfile)
{
#if WITH_EDITOR
	UserMediaProfile = InMediaProfile;
	SaveConfig();
#endif
}
