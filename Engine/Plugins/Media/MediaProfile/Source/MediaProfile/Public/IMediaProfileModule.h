// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "UObject/NameTypes.h"

class IMediaProfileManager;

/** Public interface for MediaProfileModule */
class IMediaProfileModule : public IModuleInterface
{
public:
	/** Legacy package name (/Script/MediaFrameworkUtilities) for unmigrated media profile assets; add a class path built from this alongside the current class path in asset picker filters so unmigrated entries remain discoverable. */
	static MEDIAPROFILE_API const FName LegacyPackageName;

	/** Gets the global media profile manager */
	virtual IMediaProfileManager& GetProfileManager() = 0;
	
	/** Gets whether the engine has been loaded with the necessary components to allow media captures using a level editor viewport client */
	virtual bool CanCaptureWithLevelEditorViewportClients() = 0;
};
