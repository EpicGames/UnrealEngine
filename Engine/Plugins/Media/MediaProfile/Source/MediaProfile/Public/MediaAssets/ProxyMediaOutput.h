// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaOutput.h"

#include "ProxyMediaOutput.generated.h"


/**
 * A media output that redirect to another media output.
 */
UCLASS(BlueprintType)
class MEDIAPROFILE_API UProxyMediaOutput
	: public UMediaOutput
{
	GENERATED_BODY()

	UProxyMediaOutput();

private:

	/** Dynamic media output proxy. */
	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UMediaOutput> DynamicProxy;

	/** Media output proxy. */
	UPROPERTY(EditAnywhere, Category = "Media Proxy")
	TObjectPtr<UMediaOutput> Proxy;

public:

	/**
	 * Get the media output proxy.
	 *
	 * @return The media output, or nullptr if not set.
	 */
	UMediaOutput* GetMediaOutput() const;

	/**
	 * Get the last element of the media output chain that is not a proxy media output.
	 *
	 * @return The media output, or nullptr if not set.
	 */
	UMediaOutput* GetLeafMediaOutput() const;

	/**
	 * Is the media proxy has a valid proxy.
	 *
	 * @return true if the proxy is valid.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media Proxy")
	bool IsProxyValid() const;

	/**
	 * Set the dynamic media output proxy.
	 *
	 * @param InProxy The proxy to use.
	 */
	void SetDynamicMediaOutput(UMediaOutput* InProxy);


#if WITH_EDITOR
	/**
	 * Set the media output proxy.
	 *
	 * @param InProxy The proxy to use.
	 */
	void SetMediaOutput(UMediaOutput* InProxy);
#endif

public:

	//~ UMediaOutput interface
	virtual bool Validate(FString& OutFailureReason) const override;
	virtual FIntPoint GetRequestedSize() const override;
	virtual EPixelFormat GetRequestedPixelFormat() const override;
	virtual EMediaCaptureConversionOperation GetConversionOperation(EMediaCaptureSourceType InSourceType) const override;

#if WITH_EDITOR
	virtual FString GetDescriptionString() const override;
	virtual void GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const override;
#endif //WITH_EDITOR
	
protected:

	virtual UMediaCapture* CreateMediaCaptureImpl() override;

private:

	mutable bool bLeafMediaOutput;
	mutable bool bValidateGuard;
	mutable bool bRequestedSizeGuard;
	mutable bool bRequestedPixelFormatGuard;
	mutable bool bConfigurationTextGuard;
	mutable bool bGetInfoElementsGuard;
	mutable bool bCreateMediaCaptureImplGuard;
};
