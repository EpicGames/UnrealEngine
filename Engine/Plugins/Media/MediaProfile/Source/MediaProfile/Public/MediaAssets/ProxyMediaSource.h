// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaSource.h"

#include "ProxyMediaSource.generated.h"


/**
 * A media source that redirects to another media source.
 */
UCLASS(BlueprintType)
class MEDIAPROFILE_API UProxyMediaSource
	: public UMediaSource
{
	GENERATED_BODY()

	UProxyMediaSource();

private:

	/** Cached media source proxy. */
	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UMediaSource> DynamicProxy;

	/** Media source proxy. */
	UPROPERTY(EditAnywhere, Category="Media Proxy")
	TObjectPtr<UMediaSource> Proxy;

public:

	/**
	 * Get the media source proxy.
	 *
	 * @return The media source, or nullptr if not set.
	 */
	UMediaSource* GetMediaSource() const;

	/**
	 * Get the last element of the media source chain that is not a proxy media source.
	 *
	 * @return The media source, or nullptr if not set.
	 */
	UMediaSource* GetLeafMediaSource() const;

	/**
	 * Is the media proxy has a valid proxy.
	 *
	 * @return true if the proxy is valid.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media Proxy")
	bool IsProxyValid() const;

	/**
	 * Set the dynamic media source proxy.
	 * 
	 * @param InProxy The proxy to use.
	 */
	void SetDynamicMediaSource(UMediaSource* InProxy);

#if WITH_EDITOR
	/**
	 * Set the media source proxy.
	 *
	 * @param InProxy The proxy to set.
	 */
	void SetMediaSource(UMediaSource* InProxy);
#endif

public:

	//~ UMediaSource interface

	virtual FString GetUrl() const override;
	virtual bool Validate() const override;

#if WITH_EDITOR
	virtual FString GetDescriptionString() const override;
	virtual void GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const override;
#endif //WITH_EDITOR
	
public:

	//~ IMediaOptions interface

	virtual bool GetMediaOption(const FName& Key, bool DefaultValue) const override;
	virtual double GetMediaOption(const FName& Key, double DefaultValue) const override;
	virtual int64 GetMediaOption(const FName& Key, int64 DefaultValue) const override;
	virtual FString GetMediaOption(const FName& Key, const FString& DefaultValue) const override;
	virtual FText GetMediaOption(const FName& Key, const FText& DefaultValue) const override;
	virtual TSharedPtr<IMediaOptions::FDataContainer, ESPMode::ThreadSafe> GetMediaOption(const FName& Key, const TSharedPtr<FDataContainer, ESPMode::ThreadSafe>& DefaultValue) const override;
	virtual bool HasMediaOption(const FName& Key) const override;

private:

	mutable bool bUrlGuard;
	mutable bool bValidateGuard;
	mutable bool bConfigurationTextGuard;
	mutable bool bGetInfoElementsGuard;
	mutable bool bLeafMediaSource;
	mutable bool bMediaOptionGuard;

};
