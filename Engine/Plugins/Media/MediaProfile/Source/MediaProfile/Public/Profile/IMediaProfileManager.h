// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UMediaProfile;
class UProxyMediaSource;
class UProxyMediaOutput;

class MEDIAPROFILE_API IMediaProfileManager
{
public:
	static IMediaProfileManager& Get();

	/** IMediaProfileManager structors */
	virtual ~IMediaProfileManager() {}
	
	/** Get the current profile used by the manager. Can be null. */
	virtual UMediaProfile* GetCurrentMediaProfile() const = 0;
	
	/** Set the current profile used by the manager. */
	virtual void SetCurrentMediaProfile(UMediaProfile* InMediaProfile) = 0;

	/** Get all the media source proxy. */
	virtual TArray<UProxyMediaSource*> GetAllMediaSourceProxy() const = 0;

	/** Get all the media output proxy. */
	virtual TArray<UProxyMediaOutput*> GetAllMediaOutputProxy() const = 0;

	/** Delegate type for media profile changed event */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMediaProfileChanged, UMediaProfile* /*Preivous*/, UMediaProfile* /*New*/);

	/** Delegate for media profile changed event */
	virtual FOnMediaProfileChanged& OnMediaProfileChanged() = 0;
};
