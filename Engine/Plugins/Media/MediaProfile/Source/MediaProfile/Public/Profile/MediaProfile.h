// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "MediaProfile.generated.h"

class UMediaProfilePlaybackManager;
class UEngineCustomTimeStep;
class UMediaOutput;
class UMediaSource;
class UTimecodeProvider;

/**
 * A media profile that configures the inputs, outputs, timecode provider and custom time step.
 */
UCLASS(BlueprintType)
class MEDIAPROFILE_API UMediaProfile : public UObject
{
	GENERATED_BODY()

public:
	UMediaProfile(const FObjectInitializer& ObjectInitializer);

protected:

	/** Media sources. */
	UPROPERTY(EditAnywhere, Instanced, Category="Inputs", EditFixedSize, meta=(EditFixedOrder))
	TArray<TObjectPtr<UMediaSource>> MediaSources;

	/** Media outputs. */
	UPROPERTY(EditAnywhere, Instanced, Category="Outputs", EditFixedSize, meta=(EditFixedOrder))
	TArray<TObjectPtr<UMediaOutput>> MediaOutputs;

	/** Override the Engine's Timecode provider defined in the project settings. */
	UPROPERTY(EditAnywhere, Category="Timecode Provider", meta=(DisplayName="Override Project Settings"))
	bool bOverrideTimecodeProvider;

	/** Timecode provider. */
	UPROPERTY(EditAnywhere, Instanced, Category="Timecode Provider", meta=(EditCondition="bOverrideTimecodeProvider"))
	TObjectPtr<UTimecodeProvider> TimecodeProvider;

	/** Override the Engine's Custom time step defined in the project settings. */
	UPROPERTY(EditAnywhere, Category="Genlock", meta=(DisplayName="Override Project Settings"))
	bool bOverrideCustomTimeStep;

	/** Custom time step */
	UPROPERTY(EditAnywhere, Instanced, Category="Genlock", meta=(EditCondition="bOverrideCustomTimeStep"))
	TObjectPtr<UEngineCustomTimeStep> CustomTimeStep;

#if WITH_EDITORONLY_DATA
	/** Stores user defined labels for the profile's media sources, so that users can organize and identify their media sources easier */
	UPROPERTY()
	TArray<FString> MediaSourceLabels;

	/** Stores user defined labels for the profile's media outputs, so that users can organize and identify their media outputs easier */
	UPROPERTY()
	TArray<FString> MediaOutputLabels;
#endif
	
public:
#if WITH_EDITORONLY_DATA
	/**
	 * When the profile is the current profile and modifications made it dirty.
	 * Without re-apply the profile does modification won't take effect.
	 */
	bool bNeedToBeReapplied;
#endif

public:

	/**
	 * Get the media source for the selected proxy.
	 *
	 * @return The media source, or nullptr if not set.
	 */
	UMediaSource* GetMediaSource(int32 Index) const;

	/**
	 * Sets the media source at the specified index in the profile's list of sources
	 * @param Index The index of the source to set
	 * @param InMediaSource The new media source to set
	 */
	void SetMediaSource(int32 Index, UMediaSource* InMediaSource);
	
	/**
	 * Get the number of media source.
	 */
	int32 NumMediaSources() const;

	/**
	 * Gets the index of the specified media source if it was found in the media profile's list of sources
	 * @param MediaSource The media source to get the index of
	 * @return The index of the source in the profile's sources list, or INDEX_NONE if it was not found
	 */
	int32 FindMediaSourceIndex(UMediaSource* MediaSource) const;
	
#if WITH_EDITOR
	/**
	 * Gets the index of a media source whose label matches the specified label if it was found in the media profile's list of sources
	 * @param InLabel The label of the media source to find
	 * @return The index of the source in the profile's sources list, or INDEX_NONE if it was not found
	 */
	int32 FindMediaSourceIndexByLabel(const FString& InLabel) const;
#endif

	/**
	 * Adds the specified media source to the profile's list of media sources
	 * @param MediaSource The media source to add
	 */
	void AddMediaSource(UMediaSource* MediaSource);
	
	/**
	 * Removes the specified media source from the profile's list of sources
	 * @param Index The index of the media source to remove
	 * @return true if the media source was removed
	 */
	bool RemoveMediaSource(int32 Index);

	/**
	 * Moves the media source at the specified index to the destination index, shifting other media sources down to accommodate
	 * @param CurrentIndex The current index of the media source to move
	 * @param DestIndex The index to put the media source at
	 * @return true if the move occurred, otherwise false
	 */
	bool MoveMediaSource(int32 CurrentIndex, int32 DestIndex);

#if WITH_EDITOR
	/**
	 * Gets the label for a media source
	 * @param Index The index of the media source
	 * @return The label that has been set for the media source
	 */
	FString GetLabelForMediaSource(int32 Index);

	/**
	 * Sets the label for a media source
	 * @param Index The index of the media source
	 * @param NewLabel The label to set for the media source
	 */
	void SetLabelForMediaSource(int32 Index, const FString& NewLabel);
#endif //WITH_EDITOR
	
	/**
	 * Get the media output for the selected proxy.
	 *
	 * @return The media output, or nullptr if not set.
	 */
	UMediaOutput* GetMediaOutput(int32 Index) const;

	/**
		 * Sets the media output at the specified index in the profile's list of outputs
		 * @param Index The index of the output to set
		 * @param InMediaOutput The new media output to set
		 */
	void SetMediaOutput(int32 Index, UMediaOutput* InMediaOutput);
	
	/**
	 * Get the number of media output.
	 */
	int32 NumMediaOutputs() const;

	/**
	 * Gets the index of the specified media output if it was found in the media profile's list of outputs
	 * @param MediaOutput The media output to get the index of
	 * @return The index of the output in the profile's output list, or INDEX_NONE if it was not found
	 */
	int32 FindMediaOutputIndex(UMediaOutput* MediaOutput) const;
	
#if WITH_EDITOR
	/**
	 * Gets the index of a media output with the specified label if it was found in the media profile's list of outputs
	 * @param InLabel The label of the media output to find
	 * @return The index of the output in the profile's output list, or INDEX_NONE if it was not found
	 */
	int32 FindMediaOutputIndexByLabel(const FString& InLabel) const;
#endif

	/**
	 * Adds the specified media output to the profile's list of media outputs
	 * @param MediaOutput The media output to add
	 */
	void AddMediaOutput(UMediaOutput* MediaOutput);
	
	/**
	 * Removes the specified media output from the profile's list of outputs
	 * @param Index The index of the media output to remove
	 * @return true if the media output was removed
	 */
	bool RemoveMediaOutput(int32 Index);

	/**
	 * Moves the media output at the specified index to the destination index, shifting other media outputs down to accommodate
	 * @param CurrentIndex The current index of the media output to move
	 * @param DestIndex The index to put the media output at
	 * @return true if the move occurred, otherwise false
	 */
	bool MoveMediaOutput(int32 CurrentIndex, int32 DestIndex);
	
#if WITH_EDITOR
	/**
	 * Gets the label for a media output
	 * @param Index The index of the media output
	 * @return The label that has been set for the media output
	 */
	FString GetLabelForMediaOutput(int32 Index);

	/**
	 * Sets the label for a media output
	 * @param Index The index of the media output
	 * @param NewLabel The label to set for the media output
	 */
	void SetLabelForMediaOutput(int32 Index, const FString& NewLabel);
#endif //WITH_EDITOR
	
	/**
	 * Get the timecode provider.
	 *
	 * @return The timecode provider, or nullptr if not set.
	 */
	UTimecodeProvider* GetTimecodeProvider() const;

	/**
	 * Get the custom time step.
	 *
	 * @return The custom time step, or nullptr if not set.
	 */
	UEngineCustomTimeStep* GetCustomTimeStep() const;

	/** Resets and applies the media profile's timecode to the engine if it is set to override the project settings timecode */
	void ApplyTimecodeProvider();

	/** Resets and applies the media profile's custom time step to the engine if it is set to override the project settings timecode */
	void ApplyCustomTimeStep();
	
public:

	/**
	 * Apply the media profile.
	 * Will change the engine's timecode provider & custom time step and redirect the media profile source/output proxy for the correct media source/output.
	 */
	virtual void Apply();

	/**
	 * Reset the media profile.
	 * Will reset the engine's timecode provider & custom time step and redirect the media profile source/output proxy for no media source/output.
	 */
	virtual void Reset();

	/**
	 * Update the number of sources and outputs to the number to proxies.
	 */
	void FixNumSourcesAndOutputs();

	/** Gets the playback manager to use for playback of this profile's media sources and outputs */
	UMediaProfilePlaybackManager* GetPlaybackManager() const;
	
private:

	void ResetTimecodeProvider();
	void ResetCustomTimeStep();
	void SendAnalytics() const;

private:

	/** Applied Timecode provider, cached to reset the previous value. */
	bool bTimecodeProvideWasApplied;
	UPROPERTY(Transient)
	TObjectPtr<UTimecodeProvider> AppliedTimecodeProvider;
	UPROPERTY(Transient)
	TObjectPtr<UTimecodeProvider> PreviousTimecodeProvider;

	/** Applied Custom time step, cached to reset the previous value. */
	bool bCustomTimeStepWasApplied;
	UPROPERTY(Transient)
	TObjectPtr<UEngineCustomTimeStep> AppliedCustomTimeStep;
	UPROPERTY(Transient)
	TObjectPtr<UEngineCustomTimeStep> PreviousCustomTimeStep;

	/** Playback manager with maintains any necessary objects to support playback of the profile's media sources and outputs */
	UPROPERTY(Transient, Instanced)
	TObjectPtr<UMediaProfilePlaybackManager> PlaybackManager;

	/** Allows the LLH handler to directly read/write protected fields for session serialization. */
	friend class FLiveLinkHubMediaProfileHandler;
};
