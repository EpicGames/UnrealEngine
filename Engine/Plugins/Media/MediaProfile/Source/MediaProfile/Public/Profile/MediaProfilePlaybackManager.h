// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/TimerHandle.h"
#include "Tickable.h"
#include "UObject/SoftObjectPtr.h"

#include "MediaProfilePlaybackManager.generated.h"

enum class EMediaCaptureState : uint8;
struct FMediaCaptureOptions;
struct FMediaCaptureTransform;
class FSceneViewport;
class FViewportClient;
class UMediaOutput;
class UMediaCapture;
class UMediaSource;
class UMediaTexture;
class UMediaPlayer;
class UMediaProfile;
class UTextureRenderTarget2D;

namespace MediaProfilePlaybackManager
{
	class FManagedViewport;
	class FActiveViewport;
	class FMediaTextureCapture;
	
	/**
	 * Queries a config property (MediaProfile.bAllowActiveViewportCapture, in DefaultEngine.ini) to determine if active viewport capture is allowed.
	 * If disabled, attempting to start active viewport captures in the playback manager will fail.
	 */
	MEDIAPROFILE_API bool IsActiveViewportCaptureAllowed();
	
	/**
	 * Queries a config property (MediaProfile.bAllowManagedViewportCapture, in DefaultEngine.ini) to determine if managed viewport capture is allowed.
	 * If disabled, attempting to start managed viewport captures in the playback manager will fail.
	 */
	MEDIAPROFILE_API bool IsManagedViewportCaptureAllowed();
}

/** Manages playback of media items within a media profile */
UCLASS()
class MEDIAPROFILE_API UMediaProfilePlaybackManager : public UObject, public FTickableGameObject
{
	GENERATED_BODY()

	/** Alias for callback used to restart capture of media outputs */
	using FRecaptureFunc = TFunction<void(UMediaProfilePlaybackManager*)>;
	
public:
	UMediaProfilePlaybackManager(const FObjectInitializer& ObjectInitializer);
	virtual ~UMediaProfilePlaybackManager() override;
	
public:
	//~ Begin FTickableGameObject
	virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override;
	virtual bool IsTickable() const override;
	virtual UWorld* GetTickableGameObjectWorld() const override;
	virtual TStatId GetStatId() const override;
	virtual bool IsTickableInEditor() const override { return true; }
	virtual bool IsTickableWhenPaused() const override { return true; }
	//~ End FTickableGameObject
	
	/**
	 * Gets the media texture that is outputting the specified media source
	 * @param InMediaSourceIndex The index of the media source within the media profile
	 * @return The media texture for the specified media source, or null if no media source exists at the specified index
	 */
	UMediaTexture* GetSourceMediaTextureFromIndex(int32 InMediaSourceIndex) const;

	/**
	 * Gets the media texture that is outputting the specified media source
	 * @param InMediaSource The media source within the media profile
	 * @return The media texture for the specified media source, or null if the media source is not a valid member of the media profile
	 */
	UMediaTexture* GetSourceMediaTexture(UMediaSource* InMediaSource) const;

	/**
	 * Checks the specified media texture to see if it is being used by the media profile to play one of its media sources
	 * @param InMediaTexture The media texture to test
	 * @param OutMediaSourceIndex The media source index in the media profile that is using the texture, or INDEX_NONE if there is none
	 * @return true if the media texture is being used by the active media profile; otherwise false
	 */
	bool IsValidSourceMediaTexture(UMediaTexture* InMediaTexture, int32& OutMediaSourceIndex) const;
	
	/**
	 * Opens and begins playing the specified media source 
	 * @param InMediaSourceIndex The index of the media source within the media profile
	 * @param InConsumer If not null, this consumer will be added to the list of consumers using the open media source
	 * @return The media texture that is outputting the media source, or null if no media source exists at the specified index
	 */
	UMediaTexture* OpenSourceFromIndex(int32 InMediaSourceIndex, void* InConsumer = nullptr);

	/**
	 * Opens and begins playing the specified media source 
	 * @param InMediaSource The media source within the media profile
	 * @param InConsumer If not null, this consumer will be added to the list of consumers using the open media source
	 * @return The media texture that is outputting the media source, or null if the media source is not a valid member of the media profile
	 */
	UMediaTexture* OpenSource(UMediaSource* InMediaSource, void* InConsumer = nullptr);

	/**
	 * Gets whether a media source is currently open
	 * @param InMediaSourceIndex The index of the media source within the media profile
	 * @param InConsumer If not null, additionally checks if the specified consumer is using the media source
	 * @return true if the media source is open (and being used by InConsumer, if provided); otherwise false
	 */
	bool IsSourceOpenFromIndex(int32 InMediaSourceIndex, void* InConsumer = nullptr) const;

	/**
	 * Gets whether a media source is currently open
	 * @param InMediaSource The media source within the media profile
	 * @param InConsumer If not null, additionally checks if the specified consumer is using the media source
	 * @return true if the media source is open (and being used by InConsumer, if provided); otherwise false
	 */
	bool IsSourceOpen(UMediaSource* InMediaSource, void* InConsumer = nullptr) const;
	
	/** Arguments that control what happens when a media source is closed */
	struct FCloseSourceArgs
	{
		/** The consumer requesting the media source be closed */
		void* Consumer = nullptr;

		/** Indicates that the media player should be destroyed after closing */
		bool bDestroyMediaPlayer = false;

		/** Indicates that the media source should be closed even if it still has active consumers */
		bool bForceClose = false;

		/** Indicates that the media source should stay open even if there are no active consumers */
		bool bSoftClose = false;

		FCloseSourceArgs() { }
	};
	
	/**
	 * Closes the specified media source, and optionally destroys the media player that was playing the source
	 * @param InMediaSourceIndex The index of the media source within the media profile
	 * @param InArgs Arguments that control what happens when the media source is closed
	 */
	void CloseSourceFromIndex(int32 InMediaSourceIndex, const FCloseSourceArgs& InArgs = FCloseSourceArgs());
	
	/**
	 * Closes the specified media source, and optionally destroys the media player that was playing the source
	 * @param InMediaSource The media source within the media profile
	 * @param InArgs Arguments that control what happens when the media source is closed
	 */
	void CloseSource(UMediaSource* InMediaSource, const FCloseSourceArgs& InArgs = FCloseSourceArgs());

	/**
	 * Closes all media sources that are being played by a specified consumer
	 * @param InArgs Arguments that control what happens when the media source is closed
	 */
	void CloseSourcesForConsumer(const FCloseSourceArgs& InArgs);

	/**
	 * Finds an existing or creates a new viewport to use for capture for a specified media output
	 * @param InMediaOutputIndex The index of the media output in the media profile
	 * @param InViewMode The view mode to create the viewport with
	 * @param Consumer Optional pointer to a consumer of the viewport, which ensures the viewport is not destroyed when the capture is closed
	 * @return The viewport client of the viewport of one was found or created; otherwise, null
	 */
	TSharedPtr<FViewportClient> GetOrCreateManagedViewportFromIndex(int32 InMediaOutputIndex, EViewModeIndex InViewMode, void* Consumer = nullptr);
	
	/**
	 * Finds an existing or creates a new viewport to use for capture for a specified media output
	 * @param InMediaOutput The media output in the media profile
	 * @param InViewMode The view mode to create the viewport with
	 * @param Consumer Optional pointer to a consumer of the viewport, which ensures the viewport is not destroyed when the capture is closed
	 * @return The viewport client of the viewport of one was found or created; otherwise, null
	 */
	TSharedPtr<FViewportClient> GetOrCreateManagedViewport(UMediaOutput* InMediaOutput, EViewModeIndex InViewMode, void* Consumer = nullptr);

	/**
	 * Releases a use of the managed viewport for the specified media output,which may destroy the managed viewport if it is not actively being used
	 * @param InMediaOutputIndex The index of the media output in the media profile
	 * @param Consumer The pointer of the consumer that is releasing its reference of the managed viewport
	 */
	void ReleaseManagedViewportFromIndex(int32 InMediaOutputIndex, void* Consumer);

	/**
	 * Releases a use of the managed viewport for the specified media output,which may destroy the managed viewport if it is not actively being used
	 * @param InMediaOutput The media output in the media profile
	 * @param Consumer The pointer of the consumer that is releasing its reference of the managed viewport
	 */
	void ReleaseManagedViewport(UMediaOutput* InMediaOutput, void* Consumer);

	/**
	 * Starts a capture of a managed viewport for a specified media output
	 * @param InMediaOutputIndex The index of the media output in the media profile
	 * @param InCaptureOptions The capture options to capture with
	 * @param bAutoRestartCapture Indicates that this capture will automatically restart if any modifications to the media output are detected
	 * @return The media capture object, if the capture was started successfully; otherwise, null
	 */
	UMediaCapture* OpenManagedViewportOutputFromIndex(int32 InMediaOutputIndex, const FMediaCaptureOptions& InCaptureOptions, bool bAutoRestartCapture = false);
	
	/**
	 * Starts a capture of a managed viewport for a specified media output
	 * @param InMediaOutput The media output in the media profile
	 * @param InCaptureOptions The capture options to capture with
	 * @param bAutoRestartCapture Indicates that this capture will automatically restart if any modifications to the media output are detected
	 * @return The media capture object, if the capture was started successfully; otherwise, null
	 */
	UMediaCapture* OpenManagedViewportOutput(UMediaOutput* InMediaOutput, const FMediaCaptureOptions& InCaptureOptions, bool bAutoRestartCapture = false);

	/**
	 * Gets an existing viewport in the engine that is or can be captured for the specified media output
	 * @param InMediaOutputIndex The index of the media output in the media profile
	 * @return A reference to the existing viewport, if one was found; otherwise, null
	 */
	TSharedPtr<FSceneViewport> GetActiveViewportFromIndex(int32 InMediaOutputIndex) const;
	
	/**
	 * Gets an existing viewport in the engine that is or can be captured for the specified media output
	 * @param InMediaOutput The media output in the media profile
	 * @return A reference to the existing viewport, if one was found; otherwise, null
	 */
	TSharedPtr<FSceneViewport> GetActiveViewport(UMediaOutput* InMediaOutput) const;

	/**
	 * Starts a capture of an existing engine viewport for a specified media output
	 * @param InMediaOutputIndex The index of the media output in the media profile
	 * @param InCaptureOptions The capture options to capture with
	 * @param bAutoRestartCapture Indicates that this capture will automatically restart if any modifications to the media output are detected
	 * @return The media capture object, if the capture was started successfully; otherwise, null
	 */
	UMediaCapture* OpenActiveViewportOutputFromIndex(int32 InMediaOutputIndex, const FMediaCaptureOptions& InCaptureOptions, bool bAutoRestartCapture = false);

	/**
	 * Starts a capture of an existing engine viewport for a specified media output
	 * @param InMediaOutput The media output in the media profile
	 * @param InCaptureOptions The capture options to capture with
	 * @param bAutoRestartCapture Indicates that this capture will automatically restart if any modifications to the media output are detected
	 * @return The media capture object, if the capture was started successfully; otherwise, null
	 */
	UMediaCapture* OpenActiveViewportOutput(UMediaOutput* InMediaOutput, const FMediaCaptureOptions& InCaptureOptions, bool bAutoRestartCapture = false);

	/**
	 * Starts a capture of a render target for a specified media output
	 * @param InMediaOutputIndex The index of the media output in the media profile
	 * @param InRenderTarget The render target to capture
	 * @param InCaptureOptions The capture options to capture with
	 * @param bAutoRestartCapture Indicates that this capture will automatically restart if any modifications to the media output are detected
	 * @return The media capture object, if the capture was started successfully; otherwise, null
	 */
	UMediaCapture* OpenRenderTargetOutputFromIndex(int32 InMediaOutputIndex, UTextureRenderTarget2D* InRenderTarget, const FMediaCaptureOptions& InCaptureOptions, bool bAutoRestartCapture = false);
	
	/**
	 * Starts a capture of a render target for a specified media output
	 * @param InMediaOutput The media output in the media profile
	 * @param InRenderTarget The render target to capture
	 * @param InCaptureOptions The capture options to capture with
	 * @param bAutoRestartCapture Indicates that this capture will automatically restart if any modifications to the media output are detected
	 * @return The media capture object, if the capture was started successfully; otherwise, null
	 */
	UMediaCapture* OpenRenderTargetOutput(UMediaOutput* InMediaOutput, UTextureRenderTarget2D* InRenderTarget, const FMediaCaptureOptions& InCaptureOptions, bool bAutoRestartCapture = false);

	/**
	 * Starts a capture of a media texture for a specified media output
	 * @param InMediaOutputIndex The index of the media output in the media profile
	 * @param InMediaTexture The media texture to capture
	 * @param InCaptureOptions The capture options to capture with
	 * @param InTransform The transform to apply to the media texture on output
	 * @param bAutoRestartCapture Indicates that this capture will automatically restart if any modifications to the media output are detected
	 * @return The media capture object, if the capture was started successfully; otherwise, null
	 */
	UMediaCapture* OpenMediaTextureOutputFromIndex(int32 InMediaOutputIndex, UMediaTexture* InMediaTexture, const FMediaCaptureOptions& InCaptureOptions, const FMediaCaptureTransform& InTransform, bool bAutoRestartCapture = false);
	
	/**
	 * Starts a capture of a media texture for a specified media output
	 * @param InMediaOutput The media output in the media profile
	 * @param InMediaTexture The media texture to capture
	 * @param InCaptureOptions The capture options to capture with
	 * @param InTransform The transform to apply to the media texture on output
	 * @param bAutoRestartCapture Indicates that this capture will automatically restart if any modifications to the media output are detected
	 * @return The media capture object, if the capture was started successfully; otherwise, null
	 */
	UMediaCapture* OpenMediaTextureOutput(UMediaOutput* InMediaOutput, UMediaTexture* InMediaTexture, const FMediaCaptureOptions& InCaptureOptions, const FMediaCaptureTransform& InTransform, bool bAutoRestartCapture = false);
	
	 /* 
	 * Restarts a capture of an existing engine viewport for a specified media output
	 * @param InMediaOutput The media output in the media profile
	 * @param InCaptureOptions The capture options to capture with
	 * @param bAutoRestartCapture Indicates that this capture will automatically restart if any modifications to the media output are detected
	 */
	void RestartActiveViewportOutput(UMediaOutput* InMediaOutput, FMediaCaptureOptions InCaptureOptions, bool bAutoRestartCapture = false);

	/**
	 * Restarts a capture of a managed viewport for a specified media output
	 * @param InMediaOutput The media output in the media profile
	 * @param InCaptureOptions The capture options to capture with
	 * @param bAutoRestartCapture Indicates that this capture will automatically restart if any modifications to the media output are detected
	 */
	void RestartManagedViewportOutput(UMediaOutput* InMediaOutput, FMediaCaptureOptions InCaptureOptions, bool bAutoRestartCapture = false);

	/**
	 * Restarts a capture of a render target for a specified media output
	 * @param InMediaOutput The media output in the media profile
	 * @param InRenderTarget The render target to capture
	 * @param InCaptureOptions The capture options to capture with
	 * @param bAutoRestartCapture Indicates that this capture will automatically restart if any modifications to the media output are detected
	 */
	void RestartRenderTargetOutput(UMediaOutput* InMediaOutput, UTextureRenderTarget2D* InRenderTarget, FMediaCaptureOptions InCaptureOptions, bool bAutoRestartCapture = false);

	/**
	 * Restarts a capture of a media texture for a specified media output
	 * @param InMediaOutput The media output in the media profile
	 * @param InMediaTexture The media texture to capture
	 * @param InCaptureOptions The capture options to capture with
	 * @param InTransform The transform to apply to the media texture on output
	 * @param bAutoRestartCapture Indicates that this capture will automatically restart if any modifications to the media output are detected
	 */
	void RestartMediaTextureOutput(UMediaOutput* InMediaOutput, UMediaTexture* InMediaTexture, FMediaCaptureOptions InCaptureOptions, FMediaCaptureTransform InTransform, bool bAutoRestartCapture = false);
	
	/**
	 * Gets whether there is an active capture for the specified media output index
	 * @param InMediaOutputIndex The index of the media output within the media profile to check
	 * @return true if the media output is actively being captured; otherwise false
	 */
	bool IsOutputCapturingFromIndex(int32 InMediaOutputIndex) const;
	
	/**
	 * Gets whether there is an active capture for the specified media output
	 * @param InMediaOutput The media output within the media profile to check
	 * @return true if the media output is actively being captured; otherwise false
	 */
	bool IsOutputCapturing(UMediaOutput* InMediaOutput) const;

	/**
	 * Gets the capture state of the specified media output, or null of there is no active capture of the media output
	 * @param InMediaOutputIndex The index of the media output within the media profile to check
	 * @param bOutHasError Indicates that the capture state experienced an error at some point and is stopped
	 * @return The capture state of the media output, or a null TOptional if there is no active capture of the media output
	 */
	TOptional<EMediaCaptureState> GetOutputCaptureStateFromIndex(int32 InMediaOutputIndex, bool& bOutHasError) const;

	/**
	 * Gets the capture state of the specified media output, or null of there is no active capture of the media output
	 * @param InMediaOutput The media output within the media profile to check
	 * @param bOutHasError Indicates that the capture state experienced an error at some point and is stopped
	 * @return The capture state of the media output, or a null TOptional if there is no active capture of the media output
	 */
	TOptional<EMediaCaptureState> GetOutputCaptureState(UMediaOutput* InMediaOutput, bool& bOutHasError) const;
	
	/** Arguments that control what happens when a media output is closed */
	struct FCloseOutputArgs
	{
		/** Indicates that the media capture and all related objects should be destroyed. Set to false if the capture is immediately going to be restarted. */
		bool bDestroyCaptureObjects;

		/** Some capture implementations can take a few frames to completely stop. When they are completely done, this callback delegate will be invoked. */
		FSimpleDelegate Callback;

		FCloseOutputArgs()
			: bDestroyCaptureObjects(true)
		{
		}
	};

	/**
	 * Closes the capture of the specified media output
	 * @param InMediaOutputIndex The index of the media output within the media profile
	 * @param InArgs Arguments that control what happens when the output is stopped.
	 */
	void CloseOutputFromIndex(int32 InMediaOutputIndex, const FCloseOutputArgs& InArgs = FCloseOutputArgs());

	/**
	 * Closes the capture of the specified media output
	 * @param InMediaOutput The media output in the media profile to close
	 * @param InArgs Arguments that control what happens when the output is stopped.
	 */
	void CloseOutput(UMediaOutput* InMediaOutput, const FCloseOutputArgs& InArgs = FCloseOutputArgs());
	
private:
	/**
	 * Attempts to find an existing media player for the specified source, and creates one if an existing one is not found
	 * @param InMediaSourceIndex The index of the media source within the media profile
	 * @return The media player for the media source, or null if no media source exists at the specified index
	 */
	UMediaPlayer* FindOrCreateMediaPlayerForIndex(int32 InMediaSourceIndex);

	/**
	 * Processes a change to a media source's slot index, moving any existing media players around to the correct media textures
	 * @param InOriginalMediaSourceIndex The original index of the media source, or INDEX_NONE if we are processing an insert
	 * @param InNewMediaSourceIndex The new index of the media source, or INDEX_NONE if we are processing a delete
	 */
	void ChangeMediaSourceIndex(int32 InOriginalMediaSourceIndex, int32 InNewMediaSourceIndex);
	
	/** Gets the media profile that owns this playback manager */
	UMediaProfile* GetOwningMediaProfile() const;

	/** Registers a consumer with a specific media player */ 
	void RegisterMediaSourceConsumer(UMediaPlayer* InMediaPlayer, void* InConsumer);

	/** Unregisters a consumer with a sepcific media player */
	void UnregisterMediaSourceConsumer(UMediaPlayer* InMediaPlayer, void* InConsumer);

	/** Gets the number of consumers that have registered as using the specified media player */
	int32 GetMediaSourceConsumerCount(UMediaPlayer* InMediaPlayer) const;

	/** Attempts to find an existing valid capture or creates a new capture for the specified media output index */
	UMediaCapture* FindOrCreateMediaCaptureForIndex(int32 InMediaOutputIndex);

	/**
	 * Processes a change to a media output's slot index, moving any existing media captures around
	 * @param InOriginalMediaOutputIndex The original index of the media output, or INDEX_NONE if we are processing an insert
	 * @param InNewMediaOutputIndex The new index of the media output, or INDEX_NONE if we are processing a delete
	 */
	void ChangeMediaOutputIndex(int32 InOriginalMediaOutputIndex, int32 InNewMediaOutputIndex);

	/** Raised when the capture state of the specified media capture has changed. Used to keep track of which captures may have errored */
	void OnCaptureStateChanged(UMediaCapture* InMediaCapture);

	/** Raised when a change to the specified media output has been detected and the capture shout be auto-restarted */
	void QueueRestartCapture(UMediaOutput* InMediaOutput, FRecaptureFunc InRestartCaptureFunc);

	/** Invoked through QueueRestartCapture when the StopCapture implementation is done. */
	void DeferredRestart();

	/** Called on the next available tick, restarts any captures that have been queued to auto-restart */
	void RestartCaptures();

	/** Cleans up all open sources and outputs, and deletes any managed state */
	void Cleanup();

	/** Cleans up any managed resources used by the specified media output, such as active viewports or managed viewports */
	void CleanUpCaptureResources(UMediaOutput* InMediaOutput);
	
	/** Handles cleaning up resources and invoking the post StopCapture callback. */
	void PostStopCapture(int32 InMediaOutputIndex, FCloseOutputArgs InArgs);
	
#if WITH_EDITOR
	/** Raised when a PIE session has started */
	void OnPostPIEStarted(bool bIsSimulating);

	/** Raised when a PIE session is ending */
	void OnPrePIEEnded(bool bIsSimulating);

	/** Raised when the level editor's list of viewports has changed */
	void OnLevelViewportClientListChanged();
#endif
	
private:
	/** A map of media players, where the hash key is the index of the media source the player is playing */
	UPROPERTY(Transient)
	TMap<int32, TObjectPtr<UMediaPlayer>> MediaSourcePlayers;

	/** A list of media textures to use to output the corresponding media sources */
	UPROPERTY(Transient)
	TArray<TSoftObjectPtr<UMediaTexture>> MediaSourceTextures;

	/** A map of media captures, where the hash key is the index of the media output being captured */
	UPROPERTY(Transient)
	TMap<int32, TObjectPtr<UMediaCapture>> MediaOutputCaptures;
	
	/** Keeps track of registered consumers that are actively using an open media source */
	TMap<TObjectPtr<UMediaPlayer>, TSet<void*>> SourceConsumers;
	
	/** A list of viewports being managed by the playback manager,  */
	TArray<TSharedPtr<MediaProfilePlaybackManager::FManagedViewport>> ManagedOutputViewports;

	/** A list of active viewports that are being captured from */
	TArray<TSharedPtr<MediaProfilePlaybackManager::FActiveViewport>> ActiveOutputViewports;

	/** A list of media textures that are being captured from */
	TArray<TSharedPtr<MediaProfilePlaybackManager::FMediaTextureCapture>> MediaTextureCaptures;

	/** Keeps track of which captures have reported an error with the capture */
	TSet<TObjectPtr<UMediaCapture>> CapturesWithError;
	
	/** A list of callbacks to use to auto-restart any stopped captures on the next tick */
	TArray<FRecaptureFunc> RestartCaptureFuncs;

	/** Timer handle for auto-restarting output captures */
	FTimerHandle RestartCapturesTimerHandle;
	
	friend class UMediaProfile;
};
