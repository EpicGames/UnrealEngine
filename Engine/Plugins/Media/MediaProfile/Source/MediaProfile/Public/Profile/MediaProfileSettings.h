// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "Engine/EngineTypes.h"
#include "UObject/SoftObjectPtr.h"

#include "MediaProfileSettings.generated.h"

class UMediaProfile;
class UProxyMediaOutput;
class UProxyMediaSource;


/**
 * Settings for the media profile.
 */
UCLASS(config=Game, defaultconfig)
class MEDIAPROFILE_API UMediaProfileSettings
	: public UObject
{
	GENERATED_BODY()	

public:

	/**
	 * Apply the startup media profile even when we are running a commandlet.
	 * @note We always try to apply the user media profile before the startup media profile in the editor or standalone.
	 */
	UPROPERTY(Config, EditAnywhere, Category="MediaProfile")
	bool bApplyInCommandlet;

#if WITH_EDITORONLY_DATA
	/** 
	 * Forces the media profile to save itself after a property change (if the profile was already saved once)
	 */
	UPROPERTY(Config)
	bool bAutosaveAfterChange = false;
#endif

private:

	UPROPERTY(AdvancedDisplay, Config, EditAnywhere, Category="MediaProfile")
	TArray<TSoftObjectPtr<UProxyMediaSource>> MediaSourceProxy;

	UPROPERTY(AdvancedDisplay, Config, EditAnywhere, Category="MediaProfile")
	TArray<TSoftObjectPtr<UProxyMediaOutput>> MediaOutputProxy;

	/**
	 * The media profile to use at startup.
	 * @note The media profile can be overridden in the editor by user.
	 */
	UPROPERTY(Config, EditAnywhere, Category="MediaProfile")
	TSoftObjectPtr<UMediaProfile> StartupMediaProfile;

public:

	/**
	 * Get all the media source proxy.
	 *
	 * @return The an array of the media source proxy.
	 */
	TArray<UProxyMediaSource*> LoadMediaSourceProxies() const;

	/**
	 * Get all the media output proxy.
	 *
	 * @return The an array of the media output proxy.
	 */
	TArray<UProxyMediaOutput*> LoadMediaOutputProxies() const;

	/**
	 * Get the media profile used by the engine.
	 *
	 * @return The media profile, or nullptr if not set.
	 */
	UMediaProfile* GetStartupMediaProfile() const;

#if WITH_EDITOR
	/** Set the media proxies used by the engine. */
	void SetMediaSourceProxy(const TArray<UProxyMediaSource*>& Proxies);

	/** Fills the list of proxy sources with the default proxy assets stored in the media framework plugin */
	void FillDefaultMediaSourceProxies(int32 NumSources, bool bSilent = false);
	
	/** Set the media proxies used by the engine. */
	void SetMediaOutputProxy(const TArray<UProxyMediaOutput*>& Proxies);

	/** Fills the list of proxy outputs with the default proxy assets stored in the media framework plugin */
	void FillDefaultMediaOutputProxies(int32 NumOutputs, bool bSilent = false);
	
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	/** Called when a proxy changed. */
	FSimpleMulticastDelegate OnMediaProxiesChanged;
#endif
};


/**
 * Settings for the media profile in the editor or standalone.
 * @note Cooked games don't use this media profile setting.
 */
UCLASS(config=EditorPerProjectUserSettings)
class MEDIAPROFILE_API UMediaProfileEditorSettings
	: public UObject
{
	GENERATED_BODY()

	UMediaProfileEditorSettings();

public:

#if WITH_EDITORONLY_DATA
	/**
	 * Display the media profile icon in the editor toolbar.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "MediaProfile", meta=(ConfigRestartRequired=true))
	bool bDisplayInToolbar;

	/** When enabled, the media profile name will be displayed in the main editor UI. */
	UPROPERTY(config, EditAnywhere, Category = "MediaProfile")
	bool bDisplayInMainEditor;

private:

	/**
	 * The media profile to use in standalone & editor.
	 * @note The startup media profile in the project setting will be used when in cooked game.
	 */
	UPROPERTY(Config, EditAnywhere, Category="MediaProfile")
	TSoftObjectPtr<UMediaProfile> UserMediaProfile;
#endif

public:

	/**
	 * Get the media profile used by the engine when in the editor & standalone.
	 *
	 * @return The media profile, or nullptr if not set.
	 */
	UMediaProfile* GetUserMediaProfile() const;

	/** Set the media profile used by the engine when in the editor & standalone. */
	void SetUserMediaProfile(UMediaProfile* InMediaProfile) ;
};
