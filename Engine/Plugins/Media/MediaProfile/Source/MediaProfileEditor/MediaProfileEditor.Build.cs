// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class MediaProfileEditor : ModuleRules
    {
        public MediaProfileEditor(ReadOnlyTargetRules Target) : base(Target)
        {
            PublicDependencyModuleNames.AddRange(
                new string[]
                {
                    "Core",
                }
            );

            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "AssetDefinition",
                    "CoreUObject",
                    "Engine",
                    "MediaAssets",
                    "MediaIOCore",
                    "MediaIOEditor",
                    "MediaPlayerEditor",
                    "MediaProfile",
                    "Projects",
                    "PropertyEditor",
                    "Slate",
                    "SlateCore",
                    "UnrealEd"
                }
            );
        }
    }
}