// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinition/AssetDefinition_MediaOutput.h"
#include "MediaAssets/ProxyMediaOutput.h"
#include "AssetDefinition_ProxyMediaOutput.generated.h"

UCLASS()
class UAssetDefinition_ProxyMediaOutput : public UAssetDefinition_MediaOutput
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_ProxyMediaOutput", "Proxy Media Output"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UProxyMediaOutput::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Media, NSLOCTEXT("AssetDefinition", "AssetDefinition_ProxyMediaOutputSubMenu", "Media Sources + Outputs"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
