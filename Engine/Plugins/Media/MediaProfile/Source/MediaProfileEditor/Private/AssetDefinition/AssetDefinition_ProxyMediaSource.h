// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinition/AssetDefinition_MediaSource.h"
#include "MediaAssets/ProxyMediaSource.h"
#include "AssetDefinition_ProxyMediaSource.generated.h"

UCLASS()
class UAssetDefinition_ProxyMediaSource : public UAssetDefinition_MediaSource
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_ProxyMediaSource", "Proxy Media Source"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UProxyMediaSource::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Media, NSLOCTEXT("AssetDefinition", "AssetDefinition_ProxyMediaSourceSubMenu", "Media Sources + Outputs"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
