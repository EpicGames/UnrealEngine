// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/MediaProfileFactoryNew.h"
#include "AssetTypeCategories.h"
#include "Profile/MediaProfile.h"


/* UMediaProfileFactoryNew structors
 *****************************************************************************/

UMediaProfileFactoryNew::UMediaProfileFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UMediaProfile::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory interface
 *****************************************************************************/

UObject* UMediaProfileFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UMediaProfile>(InParent, InClass, InName, Flags);
}


uint32 UMediaProfileFactoryNew::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}


bool UMediaProfileFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}
