// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/ProxyMediaOutputFactoryNew.h"
#include "AssetTypeCategories.h"
#include "MediaAssets/ProxyMediaOutput.h"


/* UProxyMediaOutputFactoryNew structors
 *****************************************************************************/

UProxyMediaOutputFactoryNew::UProxyMediaOutputFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UProxyMediaOutput::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory interface
 *****************************************************************************/

UObject* UProxyMediaOutputFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UProxyMediaOutput>(InParent, InClass, InName, Flags);
}


uint32 UProxyMediaOutputFactoryNew::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}


bool UProxyMediaOutputFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}
