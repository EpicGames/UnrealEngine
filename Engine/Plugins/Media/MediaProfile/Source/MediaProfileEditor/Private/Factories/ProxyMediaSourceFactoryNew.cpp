// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/ProxyMediaSourceFactoryNew.h"
#include "AssetTypeCategories.h"
#include "MediaAssets/ProxyMediaSource.h"


/* UProxyMediaSourceFactoryNew structors
 *****************************************************************************/

UProxyMediaSourceFactoryNew::UProxyMediaSourceFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UProxyMediaSource::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory interface
 *****************************************************************************/

UObject* UProxyMediaSourceFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UProxyMediaSource>(InParent, InClass, InName, Flags);
}


uint32 UProxyMediaSourceFactoryNew::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}


bool UProxyMediaSourceFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}
