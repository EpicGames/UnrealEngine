// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaProfileEditorModule.h"

#include "Framework/MultiBox/MultiBoxExtender.h"
#include "UI/MediaProfileEditorStyle.h"

#define LOCTEXT_NAMESPACE "FMediaProfileEditorModule"

void FMediaProfileEditorModule::StartupModule()
{
	MediaProfileMenuExtender = MakeShared<FExtender>();
	
	FMediaProfileEditorStyle::Get().Register();
}

void FMediaProfileEditorModule::ShutdownModule()
{
	FMediaProfileEditorStyle::Get().Unregister();
}

#undef LOCTEXT_NAMESPACE
    
IMPLEMENT_MODULE(FMediaProfileEditorModule, MediaProfileEditor)