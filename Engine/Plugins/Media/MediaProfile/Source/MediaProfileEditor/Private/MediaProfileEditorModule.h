// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaProfileEditorModule.h"

class FExtender;

class FMediaProfileEditorModule : public IMediaProfileEditorModule
{
public:
    virtual void StartupModule() override;
    virtual void ShutdownModule() override;
    
    virtual TSharedPtr<FExtender> GetMediaProfileMenuExtender() const override { return MediaProfileMenuExtender; }
    
private:
    TSharedPtr<FExtender> MediaProfileMenuExtender;
};
