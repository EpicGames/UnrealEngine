// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaProfileEditor/Public/MediaProfileMenus.h"

#include "AssetToolsModule.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "IMediaProfileEditorModule.h"
#include "IContentBrowserSingleton.h"
#include "IMediaProfileModule.h"
#include "SMediaProfileSourceTexturePicker.h"
#include "Factories/MediaProfileFactoryNew.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/FeedbackContext.h"
#include "Profile/IMediaProfileManager.h"
#include "Profile/MediaProfile.h"
#include "Profile/MediaProfileSettings.h"
#include "Styling/ToolBarStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "UI/MediaProfileEditorStyle.h"
#include "Widgets/Images/SImage.h"
#include "Misc/ConfigCacheIni.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "MediaProfileMenus"

namespace UE::MediaProfile::Private
{
	class SMediaProfileToolBarButton : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SMediaProfileToolBarButton) {}
			SLATE_ARGUMENT(FText, ToolTipText)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs)
		{
			const FToolBarStyle& ToolbarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolbar");

			SetToolTipText(InArgs._ToolTipText);

			ChildSlot
			[
				SNew(SHorizontalBox)
				
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Fill)
				[
					SNew(SBorder)
					.BorderImage(this, &SMediaProfileToolBarButton::GetButtonBorderImage)
					.Padding(0.0f)		
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SAssignNew(Button, SButton)
						.ContentPadding(0.f)
						.ButtonStyle(&ToolbarStyle.SettingsComboButton.ButtonStyle)
						.OnClicked(this, &SMediaProfileToolBarButton::OnButtonClicked)
						[
							SNew(SImage)
							.Image(FMediaProfileEditorStyle::Get().GetBrush(TEXT("ClassIcon.MediaProfile")))
							.DesiredSizeOverride(FVector2D(16.0f, 16.0f))
						]
					]
				]
				
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Fill)
				[
					SNew(SBorder)
					.BorderImage(this, &SMediaProfileToolBarButton::GetDropdownBorderImage)
					.Padding(0.0f)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SAssignNew(Dropdown, SComboButton)
						.ContentPadding(0.f)
						.ComboButtonStyle(&ToolbarStyle.SettingsComboButton)
						.OnGetMenuContent_Static(&Menus::GenerateMediaProfileDropdownMenu)
					]
				]
			];
		}
		
	private:
		FReply OnButtonClicked()
		{
			Menus::OpenExistingOrCreateNewMediaProfile();
			return FReply::Handled();
		}
		
		const FSlateBrush* GetButtonBorderImage() const
		{
			if (Button->IsHovered())
			{
				return FAppStyle::Get().GetBrush("ToolbarSettingsRegion.LeftHover");
			}
			
			if (Dropdown->IsHovered())
			{
				return FAppStyle::Get().GetBrush("ToolbarSettingsRegion.Left");
			}
			
			return FStyleDefaults::GetNoBrush();
		}
		
		const FSlateBrush* GetDropdownBorderImage() const
		{
			if (Dropdown->IsHovered())
			{
				return FAppStyle::Get().GetBrush("ToolbarSettingsRegion.RightHover");
			}
			
			if (Button->IsHovered())
			{
				return FAppStyle::Get().GetBrush("ToolbarSettingsRegion.Right");
			}
			
			return FStyleDefaults::GetNoBrush();
		}
		
	private:
		TSharedPtr<SButton> Button;
		TSharedPtr<SComboButton> Dropdown;
	};
	
	/** Returns true if the config specifies that MediaProfile should use transient (in-memory) profiles.
	 *  Cached on first call since this value never changes at runtime. */
	bool ShouldUseTransientProfile()
	{
		static TOptional<bool> CachedValue;
		if (!CachedValue.IsSet())
		{
			bool bUseTransientProfile = false;
			GConfig->GetBool(TEXT("MediaProfile"), TEXT("bUseTransientProfile"), bUseTransientProfile, GEngineIni);
			CachedValue = bUseTransientProfile;
		}
		return CachedValue.GetValue();
	}

	void CreateNewMediaProfile()
	{
		// When configured for transient profiles (e.g. in Live Link Hub), use the existing
		// transient profile if one is active, otherwise create a new one. This avoids
		// duplicating transient objects when the user clicks "Create Media Profile" repeatedly.
		if (ShouldUseTransientProfile())
		{
			UMediaProfile* ExistingProfile = IMediaProfileManager::Get().GetCurrentMediaProfile();
			if (ExistingProfile && ExistingProfile->GetPackage() == GetTransientPackage())
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ExistingProfile);
				return;
			}

			UMediaProfile* TransientProfile = NewObject<UMediaProfile>(GetTransientPackage());
			IMediaProfileManager::Get().SetCurrentMediaProfile(TransientProfile);
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(TransientProfile);
			return;
		}

		UMediaProfileFactoryNew* FactoryInstance = DuplicateObject<UMediaProfileFactoryNew>(GetDefault<UMediaProfileFactoryNew>(), GetTransientPackage());
		UMediaProfile* NewAsset = Cast<UMediaProfile>(FAssetToolsModule::GetModule().Get().CreateAssetWithDialog(UMediaProfile::StaticClass(), FactoryInstance));
		if (NewAsset != nullptr)
		{
			GetMutableDefault<UMediaProfileEditorSettings>()->SetUserMediaProfile(NewAsset);
			IMediaProfileManager::Get().SetCurrentMediaProfile(NewAsset);
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewAsset);
		}
	}
	
	void SelectExistingMediaProfile(const FAssetData& AssetData)
	{
		FSlateApplication::Get().DismissAllMenus();

		GWarn->BeginSlowTask(LOCTEXT("MediaProfileLoadPackage", "Loading Media Profile"), true, false);
		UMediaProfile* Asset = Cast<UMediaProfile>(AssetData.GetAsset());
		GWarn->EndSlowTask();

		GetMutableDefault<UMediaProfileEditorSettings>()->SetUserMediaProfile(Asset);
		IMediaProfileManager::Get().SetCurrentMediaProfile(Asset);
	}
	
	void GenerateExistingMediaProfilesSubMenu(FMenuBuilder& MenuBuilder)
	{
		// The contents of this menu are added as a custom widget with its own search field so we
		// disable searching in this parent menu to avoid displaying two search fields to the user.
		MenuBuilder.SetSearchable(false);

		UMediaProfile* CurrentMediaProfile = IMediaProfileManager::Get().GetCurrentMediaProfile();
		const FAssetData CurrentAssetData = CurrentMediaProfile ? FAssetData(CurrentMediaProfile) : FAssetData();

		if (CurrentMediaProfile)
		{
			MenuBuilder.BeginSection(NAME_None, LOCTEXT("CurrentAssetOperationsHeader", "Current Asset"));
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("EditAsset", "Edit"),
					LOCTEXT("EditAsset_Tooltip", "Edit this asset"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"),
					FUIAction(FExecuteAction::CreateLambda([]()
					{
						if (UMediaProfile* Profile = IMediaProfileManager::Get().GetCurrentMediaProfile())
						{
							GEditor->EditObject(Profile);
						}
					}))
				);

				MenuBuilder.AddMenuEntry(
					LOCTEXT("ClearAsset", "Clear"),
					LOCTEXT("ClearAsset_ToolTip", "Clears the current Media Profile"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Delete"),
					FUIAction(FExecuteAction::CreateLambda([]()
					{
						GetMutableDefault<UMediaProfileEditorSettings>()->SetUserMediaProfile(nullptr);
						IMediaProfileManager::Get().SetCurrentMediaProfile(nullptr);
					}))
				);
			}
			MenuBuilder.EndSection();
		}

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("BrowseHeader", "Browse"));
		{
			FAssetPickerConfig AssetPickerConfig;
			AssetPickerConfig.Filter.ClassPaths.Add(UMediaProfile::StaticClass()->GetClassPathName());
			// Include the legacy class path so unmigrated assets remain discoverable without requiring a resave.
			AssetPickerConfig.Filter.ClassPaths.Add(FTopLevelAssetPath(IMediaProfileModule::LegacyPackageName, UMediaProfile::StaticClass()->GetFName()));
			AssetPickerConfig.Filter.bRecursiveClasses = true;
			AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateStatic(&SelectExistingMediaProfile);
			AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateLambda([CurrentAssetData](const FAssetData& InAssetData){ return InAssetData == CurrentAssetData; });
			AssetPickerConfig.InitialAssetSelection = CurrentAssetData;
			AssetPickerConfig.bAllowDragging = false;
			AssetPickerConfig.bAllowNullSelection = false;
			AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
			AssetPickerConfig.SaveSettingsName = TEXT("MediaProfilePicker");

			FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

			MenuBuilder.AddWidget(
				SNew(SBox)
				.WidthOverride(320.0f)
				.HeightOverride(450.0f)
				[
					ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
				],
				FText::GetEmpty(),
				true
			);
		}
		MenuBuilder.EndSection();
	}
}

void UE::MediaProfile::Menus::OpenExistingOrCreateNewMediaProfile()
{
	if (UMediaProfile* MediaProfile = IMediaProfileManager::Get().GetCurrentMediaProfile())
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(MediaProfile);
	}
	else
	{
		Private::CreateNewMediaProfile();
	}
}

TSharedRef<SWidget> UE::MediaProfile::Menus::GenerateMediaProfileDropdownMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr, IMediaProfileEditorModule::Get().GetMediaProfileMenuExtender());

	const bool bTransientMode = Private::ShouldUseTransientProfile();

	MenuBuilder.BeginSection("NewMediaProfile", LOCTEXT("NewMediaProfileSection", "New"));
	{
		// In transient mode the button opens the existing profile; in normal mode it creates a new .uasset.
		MenuBuilder.AddMenuEntry(
			bTransientMode ? LOCTEXT("OpenMediaProfileLabel", "Open Media Profile") : LOCTEXT("CreateMenuLabel", "Create Media Profile"),
			bTransientMode ? LOCTEXT("OpenMediaProfileTooltip", "Open the Media Profile editor.") : LOCTEXT("CreateMenuTooltip", "Create a new Media Profile asset."),
			FSlateIcon(FMediaProfileEditorStyle::Get().GetStyleSetName(), TEXT("ClassIcon.MediaProfile")),
			FUIAction(
				FExecuteAction::CreateStatic(&Private::CreateNewMediaProfile)
			)
		);
	}
	MenuBuilder.EndSection();

	// The asset browser section is only useful when profiles are saved to disk.
	if (!bTransientMode)
	{
		MenuBuilder.BeginSection("MediaProfile", LOCTEXT("MediaProfileSection", "Media Profile"));
		{
			UMediaProfile* Profile = IMediaProfileManager::Get().GetCurrentMediaProfile();
		
			MenuBuilder.AddSubMenu(
				TAttribute<FText>::CreateLambda([]
				{
					if (UMediaProfile* Profile = IMediaProfileManager::Get().GetCurrentMediaProfile())
					{
						return FText::FromName(Profile->GetFName());
					}
					
					return LOCTEXT("SelectMenuLabel", "Select a Media Profile");
				}),
				LOCTEXT("SelectMenuTooltip", "Select the current profile for this editor."),
				FNewMenuDelegate::CreateStatic(&Private::GenerateExistingMediaProfilesSubMenu),
				FUIAction(
					FExecuteAction(),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([]
					{
						return IMediaProfileManager::Get().GetCurrentMediaProfile() != nullptr;
					})
				),
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}
		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> UE::MediaProfile::Menus::CreateMediaProfileToolBarButton(const FText& InToolTipText)
{
	return SNew(Private::SMediaProfileToolBarButton)
		.ToolTipText(InToolTipText);
}

#undef LOCTEXT_NAMESPACE
