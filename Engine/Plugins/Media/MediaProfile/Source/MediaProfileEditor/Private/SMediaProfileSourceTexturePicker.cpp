// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaProfileEditor/Public/SMediaProfileSourceTexturePicker.h"

#include "Editor.h"
#include "MediaSource.h"
#include "MediaTexture.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Profile/IMediaProfileManager.h"
#include "Profile/MediaProfile.h"
#include "Profile/MediaProfilePlaybackManager.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "UI/MediaProfileEditorStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "SMediaProfileSourceTexturePicker"

void SMediaProfileSourceTexturePicker::Construct(const FArguments& InArgs)
{
	TexturePropertyHandle = InArgs._TexturePropertyHandle;
	OnMediaSourceSelected = InArgs._OnMediaSourceSelected;
	
	UClass* EffectiveAllowedClass = InArgs._AllowedClass ? InArgs._AllowedClass : UMediaTexture::StaticClass();
	const FOnShouldFilterAsset FilterDelegate = InArgs._OnShouldFilterAsset.IsBound()
		? InArgs._OnShouldFilterAsset
		: FOnShouldFilterAsset::CreateLambda([EffectiveAllowedClass](FAssetData const& AssetData)
		{
			if (UClass* AssetClass = AssetData.GetClass())
			{
				return !AssetClass->IsChildOf(EffectiveAllowedClass);
			}
			return true;
		});

	TSharedRef<SHorizontalBox> ButtonsBox = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SComboButton)
			.HasDownArrow(true)
			.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
			.ToolTipText(LOCTEXT("MediaProfileSourceSelectorToolTip", "Select media texture from a media source in the active Media Profile"))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnGetMenuContent(this, &SMediaProfileSourceTexturePicker::GetMediaProfileSourceSelectorMenu)
			.IsEnabled(this, &SMediaProfileSourceTexturePicker::HasActiveMediaProfile)
			.ButtonContent()
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(16.0f, 16.0f))
				.Image(FMediaProfileEditorStyle::Get().GetBrush("ClassIcon.MediaProfile"))
			]
		];

	if (TSharedPtr<SWidget> AdditionalWidget = InArgs._AdditionalContent.Widget)
	{
		int32 SlotIndex = InArgs._AdditionalContentBeforeMediaButton ? 0 : INDEX_NONE;
		ButtonsBox->InsertSlot(SlotIndex)
		.AutoWidth()
		[
			AdditionalWidget.ToSharedRef()
		];
	}

	SObjectPropertyEntryBox::FArguments BaseArgs;
	BaseArgs
		.PropertyHandle(TexturePropertyHandle)
		.AllowedClass(EffectiveAllowedClass)
		.AllowCreate(true)
		.ThumbnailPool(InArgs._ThumbnailPool)
		.OnShouldFilterAsset(FilterDelegate)
		.CustomContentSlot()
		[
			ButtonsBox
		];

	SObjectPropertyEntryBox::Construct(BaseArgs);
}

TSharedRef<SWidget> SMediaProfileSourceTexturePicker::GetMediaProfileSourceSelectorMenu()
{
	UMediaProfile* ActiveMediaProfile = IMediaProfileManager::Get().GetCurrentMediaProfile();
	if (!ActiveMediaProfile)
	{
		return SNullWidget::NullWidget;
	}
	
	const bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("MediaProfileSourcesSectionLabel", "Texture from Media Profile"));
	{
		bool bNoMediaSources = true;
		for (int32 Index = 0; Index < ActiveMediaProfile->NumMediaSources(); ++Index)
		{
			if (UMediaSource* MediaSource = ActiveMediaProfile->GetMediaSource(Index))
			{
				bNoMediaSources = false;

				MenuBuilder.AddMenuEntry(
					FText::FromString(ActiveMediaProfile->GetLabelForMediaSource(Index)),
					FText::GetEmpty(),
					FSlateIconFinder::FindIconForClass(MediaSource->GetClass()),
					FUIAction(FExecuteAction::CreateSP(this, &SMediaProfileSourceTexturePicker::SetTextureToMediaProfileSource, Index))
				);
			}
		}

		if (bNoMediaSources)
		{
			TSharedRef<SWidget> TextBlock = SNew(STextBlock)
				.Text(LOCTEXT("NoMediaSources", "No Media Sources"))
				.Font(FCoreStyle::GetDefaultFontStyle("Italic", 10));
			
			MenuBuilder.AddWidget(TextBlock,FText::GetEmpty());
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.AddSeparator();
	MenuBuilder.BeginSection(NAME_None);
	{
		MenuBuilder.AddMenuEntry(
			FText::Format(LOCTEXT("OpenActiveMediaProfileLabelFormat", "Open {0}"), FText::FromString(ActiveMediaProfile->GetName())),
			FText::GetEmpty(),
			FSlateIconFinder::FindIconForClass(ActiveMediaProfile->StaticClass()),
			FUIAction(FExecuteAction::CreateSP(this, &SMediaProfileSourceTexturePicker::OpenMediaProfile))
		);
	}
	MenuBuilder.EndSection();
	
	return MenuBuilder.MakeWidget();
}

bool SMediaProfileSourceTexturePicker::HasActiveMediaProfile() const
{
	UMediaProfile* ActiveMediaProfile = IMediaProfileManager::Get().GetCurrentMediaProfile();
	return IsValid(ActiveMediaProfile);
}

void SMediaProfileSourceTexturePicker::SetTextureToMediaProfileSource(int32 InMediaSourceIndex)
{
	UMediaProfile* ActiveMediaProfile = IMediaProfileManager::Get().GetCurrentMediaProfile();
	if (!ActiveMediaProfile)
	{
		return;
	}

	UMediaSource* MediaSource = ActiveMediaProfile->GetMediaSource(InMediaSourceIndex);
	if (MediaSource == nullptr)
	{
		return;
	}

	UMediaTexture* MediaTexture = ActiveMediaProfile->GetPlaybackManager()->GetSourceMediaTextureFromIndex(InMediaSourceIndex);
	TexturePropertyHandle->SetValue(MediaTexture);

	OnMediaSourceSelected.ExecuteIfBound(ActiveMediaProfile, InMediaSourceIndex);
}

void SMediaProfileSourceTexturePicker::OpenMediaProfile()
{
	UMediaProfile* ActiveMediaProfile = IMediaProfileManager::Get().GetCurrentMediaProfile();
	if (!ActiveMediaProfile)
	{
		return;
	}
	
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ActiveMediaProfile);
}

#undef LOCTEXT_NAMESPACE
