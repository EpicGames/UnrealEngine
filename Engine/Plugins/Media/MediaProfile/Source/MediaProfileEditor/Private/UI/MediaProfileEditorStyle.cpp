// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/MediaProfileEditorStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"

FMediaProfileEditorStyle& FMediaProfileEditorStyle::Get()
{
	static FMediaProfileEditorStyle Instance;
	return Instance;
}

void FMediaProfileEditorStyle::Register()
{
	FSlateStyleRegistry::RegisterSlateStyle(Get());
}

void FMediaProfileEditorStyle::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(Get());
}

FMediaProfileEditorStyle::FMediaProfileEditorStyle()
	: FSlateStyleSet("MediaProfileEditorStyle")
{
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);
	
	const FString ContentDir = FPaths::EnginePluginsDir() / TEXT("Media/MediaProfile/Resources");
	FSlateStyleSet::SetContentRoot(ContentDir);
	FSlateStyleSet::SetCoreContentRoot(FPaths::EngineContentDir());

	Set("ClassThumbnail.ProxyMediaOutput", new IMAGE_BRUSH("ProxyMediaOutput_64x", Icon64x64));
	Set("ClassIcon.ProxyMediaOutput", new IMAGE_BRUSH("ProxyMediaOutput_16x", Icon16x16));
	Set("ClassThumbnail.ProxyMediaSource", new IMAGE_BRUSH("ProxyMediaSource_64x", Icon64x64));
	Set("ClassIcon.ProxyMediaSource", new IMAGE_BRUSH("ProxyMediaSource_16x", Icon16x16));
	Set("ClassThumbnail.MediaProfile", new IMAGE_BRUSH("MediaProfile_64x", Icon64x64));
	Set("ClassIcon.MediaProfile", new IMAGE_BRUSH_SVG("MediaProfile", Icon20x20));
}
