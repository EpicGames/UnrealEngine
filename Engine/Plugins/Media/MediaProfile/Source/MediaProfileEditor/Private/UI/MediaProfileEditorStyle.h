// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FMediaProfileEditorStyle : public FSlateStyleSet
{
public:
	static FMediaProfileEditorStyle& Get();
	
	void Register();
	void Unregister();
	
private:
	FMediaProfileEditorStyle();
};
