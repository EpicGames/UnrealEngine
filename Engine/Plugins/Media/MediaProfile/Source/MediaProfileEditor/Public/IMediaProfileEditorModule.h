// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class FExtender;

class IMediaProfileEditorModule : public IModuleInterface
{
public:
	static IMediaProfileEditorModule& Get()
	{
		static FName ModuleName("MediaProfileEditor");
		return FModuleManager::LoadModuleChecked<IMediaProfileEditorModule>(ModuleName);
	}
	
	/** Gets a menu extender that allows the media profile toolbar/dropdown menu to be extended */
	virtual TSharedPtr<FExtender> GetMediaProfileMenuExtender() const = 0;
};
