// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

#define UE_API MEDIAPROFILEEDITOR_API

class SWidget;

namespace UE::MediaProfile::Menus
{
	/**
	 * If there is an existing media profile set as the current profile, opens that media profile in the media profile editor.
	 * Otherwise, opens the asset creation dialog box to allow users to create a new media profile asset
	 */
	UE_API void OpenExistingOrCreateNewMediaProfile();
	
	/** Creates a dropdown menu widget populated with contents related to the current media profile */
	UE_API TSharedRef<SWidget> GenerateMediaProfileDropdownMenu();
	
	/** Creates a toolbar dropdown button widget populated with contents related to the current media profile */
	UE_API TSharedRef<SWidget> CreateMediaProfileToolBarButton(const FText& InToolTipText = FText::GetEmpty());
}

#undef UE_API
