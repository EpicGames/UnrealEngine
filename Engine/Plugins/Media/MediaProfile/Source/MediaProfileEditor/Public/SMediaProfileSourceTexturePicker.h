// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyCustomizationHelpers.h"

#define UE_API MEDIAPROFILEEDITOR_API

class UMediaProfile;

/**
 * A SObjectPropertyEntryBox for picking a texture that adds a dropdown for picking the media texture from a media source in the active media profile
 */
class SMediaProfileSourceTexturePicker : public SObjectPropertyEntryBox
{
public:
	DECLARE_DELEGATE_TwoParams(FOnMediaSourceSelected, UMediaProfile* /* InMediaProfile */, int32 /* InMediaSourceIndex */);
	
public:
	SLATE_BEGIN_ARGS(SMediaProfileSourceTexturePicker)
		: _AllowedClass(nullptr)
		{ }
		SLATE_ARGUMENT(TSharedPtr<IPropertyHandle>, TexturePropertyHandle)
		SLATE_ARGUMENT(TSharedPtr<FAssetThumbnailPool>, ThumbnailPool)
		/** Overrides the default UMediaTexture allowed class for the asset picker. */
		SLATE_ARGUMENT(UClass*, AllowedClass)
		/** Overrides the default MediaTexture-only asset filter. When unbound, only UMediaTexture assets are shown. */
		SLATE_EVENT(FOnShouldFilterAsset, OnShouldFilterAsset)
		SLATE_EVENT(FOnMediaSourceSelected, OnMediaSourceSelected)
		/** Optional extra widget appended after the media profile button in the custom content slot. */
		SLATE_NAMED_SLOT(FArguments, AdditionalContent)
		/** Indicates that the additional content should be placed before the media button instead of after */
		SLATE_ARGUMENT(bool, AdditionalContentBeforeMediaButton)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs);

private:
	/** Constructs the dropdown menu that shows all the media sources in the active media profile */
	TSharedRef<SWidget> GetMediaProfileSourceSelectorMenu();
	
	/** Gets whether there is a valid active media profile */
	bool HasActiveMediaProfile() const;
	
	/** Raised when the user has selected a media source whose media texture should be used from the media sources dropdown */
	void SetTextureToMediaProfileSource(int32 InMediaSourceIndex);
	
	/** Opens the active media profile in the media profile editor */
	void OpenMediaProfile();

private:
	/** The property handle to store the texture in */
	TSharedPtr<IPropertyHandle> TexturePropertyHandle;

	/** Raised when a media source is selected */
	FOnMediaSourceSelected OnMediaSourceSelected;
};

#undef UE_API
